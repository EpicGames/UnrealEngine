{
  "FileVersion" : 3,
  "Version" : 1,
  "VersionName" : "1.0",
  "FriendlyName" : "Skeletal Mesh Simplifier (Early Access)",
  "Description" : "A plugin to generate LOD for deforming meshes.",
  "Category" : "Editor",
  "CreatedBy" : "Epic Games, Inc.",
  "CreatedByURL" : "https://epicgames.com",
  "DocsURL" : "",
  "MarketplaceURL" : "",
  "SupportURL" : "",
  "EnabledByDefault" : true,
  "CanContainContent" : false,
  "IsBetaVersion" : false,
  "Installed" : false,
  "Modules" :
	[
		{
			"Name" : "SkeletalMeshReduction",
			"Type" : "Editor",
			"LoadingPhase" : "Default",
			"PlatformAllowList": [ "Win64", "Mac", "Linux"]
		}
	]
}