// Copyright Epic Games, Inc. All Rights Reserved.



#include "AnimationRuntime.h"
#include "Animation/AttributesRuntime.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/SkinnedAssetCommon.h"
#include "Features/IModularFeatures.h"
#include "ISkeletalMeshReduction.h"
#include "MeshBoneReduction.h"
#include "Rendering/SkeletalMeshModel.h"
#include "SkeletalSimplifier.h"
#include "SkeletalMeshReductionSkinnedMesh.h"
#include "ClothingAsset.h"
#include "LODUtilities.h"
#include "Animation/AnimSequence.h"
#include "Animation/AnimSequenceHelpers.h"
#include "Async/Async.h"

#include "Animation/AnimSequence.h"
#include "BonePose.h"
#include "BoneWeights.h"

#define LOCTEXT_NAMESPACE "SkeletalMeshReduction"


class FQuadricSkeletalMeshReduction : public IMeshReduction
{
public:
	FQuadricSkeletalMeshReduction() 
	{};

	virtual ~FQuadricSkeletalMeshReduction()
	{}

	virtual const FString& GetVersionString() const override
	{
		// NB: The version string must be of the form "QuadricSkeletalMeshReduction_{foo}"
		// for the SkeletalMeshReductionSettingDetails to recognize this.
		// Version corresponds to VersionName in SkeletalReduction.uplugin.  
		static FString Version = TEXT("QuadricSkeletalMeshReduction_V0.1");
		return Version;
	}

	/**
	*	Returns true if mesh reduction is supported
	*/
	virtual bool IsSupported() const { return true; }

	/**
	*	Returns true if mesh reduction is active. Active mean there will be a reduction of the vertices or triangle number
	*/
	virtual bool IsReductionActive(const struct FMeshReductionSettings &ReductionSettings) const
	{
		return false;
	}

	virtual bool IsReductionActive(const struct FMeshReductionSettings& ReductionSettings, uint32 NumVertices, uint32 NumTriangles) const
	{
		return false;
	}

	virtual bool IsReductionActive(const FSkeletalMeshOptimizationSettings &ReductionSettings) const
	{
		return IsReductionActive(ReductionSettings, 0, 0);
	}

	virtual bool IsReductionActive(const struct FSkeletalMeshOptimizationSettings &ReductionSettings, uint32 NumVertices, uint32 NumTriangles) const
	{
		float Threshold_One = (1.0f - KINDA_SMALL_NUMBER);
		float Threshold_Zero = (0.0f + KINDA_SMALL_NUMBER);
		switch (ReductionSettings.TerminationCriterion)
		{
			case SkeletalMeshTerminationCriterion::SMTC_NumOfTriangles:
			{
				return ReductionSettings.NumOfTrianglesPercentage < Threshold_One || ReductionSettings.MaxNumOfTrianglesPercentage < NumTriangles;
			}
			case SkeletalMeshTerminationCriterion::SMTC_NumOfVerts:
			{
				return ReductionSettings.NumOfVertPercentage < Threshold_One || ReductionSettings.MaxNumOfVertsPercentage < NumVertices;
			}
			case SkeletalMeshTerminationCriterion::SMTC_TriangleOrVert:
			{
				return ReductionSettings.NumOfTrianglesPercentage < Threshold_One || ReductionSettings.NumOfVertPercentage < Threshold_One;
			}
			//Absolute count is consider has being always reduced
			case SkeletalMeshTerminationCriterion::SMTC_AbsNumOfVerts:
			{
				return ReductionSettings.MaxNumOfVerts < NumVertices;
			}
			case SkeletalMeshTerminationCriterion::SMTC_AbsNumOfTriangles:
			{
				return ReductionSettings.MaxNumOfTriangles < NumTriangles;
			}
			case SkeletalMeshTerminationCriterion::SMTC_AbsTriangleOrVert:
			{
				return ReductionSettings.MaxNumOfVerts < NumVertices || ReductionSettings.MaxNumOfTriangles < NumTriangles;
			}
		}

		return false;
	}

	/**
	* Reduces the provided skeletal mesh.
	* @returns true if reduction was successful.
	*/
	virtual bool ReduceSkeletalMesh( class USkeletalMesh* SkeletalMesh,
		                             int32 LODIndex,
									 const ITargetPlatform* TargetPlatform
	                                ) override ;


	/**
	* Reduces the raw mesh using the provided reduction settings.
	* @param OutReducedMesh - Upon return contains the reduced mesh.
	* @param OutMaxDeviation - Upon return contains the maximum distance by which the reduced mesh deviates from the original.
	* @param InMesh - The mesh to reduce.
	* @param ReductionSettings - Setting with which to reduce the mesh.
	*/
	virtual void ReduceMeshDescription( FMeshDescription& OutReducedMesh,
		                                float& OutMaxDeviation,
		                                const FMeshDescription& InMesh,
		                                const FOverlappingCorners& InOverlappingCorners,
		                                const struct FMeshReductionSettings& ReductionSettings
										) override {};

	


private:
	// -- internal structures used to for book-keeping


	/**
	* Holds data needed to create skeletal mesh skinning streams.
	*/
	struct  FSkeletalMeshData;

	/**
	* Useful in book-keeping ranges within an array.
	*/
	struct  FSectionRange;

	/**
	* Important bones and sections when simplifying
	*/
	struct FPrioritizedFeatures;

	/*
	 * Vertex attribute information
	 */
	struct FVertexAttributeInfo;
private:

	/**
	*  Reduce the skeletal mesh
	*  @param SkeletalMesh      - the mesh to be reduced.
	*  @param SkeletalMeshModel - the Model that belongs to the skeletal mesh, i.e. SkeletalMesh->GetImportedModel();
	*  @param LODIndex          - target index for the LOD   
	*/
	void ReduceSkeletalMesh( USkeletalMesh& SkeletalMesh, FSkeletalMeshModel& SkeletalMeshModel, int32 LODIndex, const ITargetPlatform* TargetPlatform) const ;


	/**
	*  Reduce the skeletal mesh
	*  @param SrcLODModel       - Input mesh for simplification
	*  @param OutLODModel       - The result of simplification
	*  @param SkeletalMeshName  - Name of the skeletal mesh
	*  @param Bounds            - The bounds of the source geometry - can be used in computing simplification error threshold
	*  @param RefSkeleton       - The reference skeleton
	*  @param Settings          - Settings that control the reduction
	*  @param ImportantBones    - Bones and associated weight ( inhibit edge collapse of verts that rely on these bones)
	*  @param BoneMatrices      - Used to pose the mesh.
	*  @param LODIndex          - Target LOD index
	*/
	bool ReduceSkeletalLODModel( const FSkeletalMeshLODModel& SrcLODModel,
		                         FSkeletalMeshLODModel& OutLODModel,
								 const FString& SkeletalMeshName,
		                         const FBoxSphereBounds& Bounds,
		                         const FReferenceSkeleton& RefSkeleton,
		                         FSkeletalMeshOptimizationSettings Settings,
								 const FPrioritizedFeatures& PrioritizedFeatures,
		                         const TArray<FMatrix>& BoneMatrices,
		                         const int32 LODIndex,
								 const bool bReducingSourceModel,
								 const ITargetPlatform* TargetPlatform) const;

	/**
	* Remove the specified section from the mesh.
	* @param Model        - the model from which we remove the section
	* @param SectionIndex - the number of the section to remove
	*/
	bool RemoveMeshSection( FSkeletalMeshLODModel& Model, int32 SectionIndex ) const;


	/**
	* Generate a representation of the skined mesh in pose prescribed by Bone Weights and Matrices with attribute data on the verts for simplification
	* @param SrcLODModel    - the original model
	* @param BoneMatrices   - define the configuration of the model
	* @param LODIndex       - target index for the result.
	* @param OutSkinnedMesh - the posed mesh
	* @param VertexAttributeInfos The vertex attribute information from the original and how it is packed in the skinned mesh.
	* @param VerToSectionMap - if provided, on return this array will allow VertexID to SectionID lookup
	*/
	void ConvertToFSkinnedSkeletalMesh( const FSkeletalMeshLODModel& SrcLODModel,
		                                const TArray<FMatrix>& BoneMatrices,
		                                const int32 LODIndex,
		                                SkeletalSimplifier::FSkinnedSkeletalMesh& OutSkinnedMesh,
										const FString& SkeletalMeshName,
										TArray<FVertexAttributeInfo>& VertexAttributeInfos, 
										TArray<int32>* VertToSectionMap = nullptr) const;

	/**
	* Generate a SkeletalMeshLODModel from a SkinnedSkeletalMesh and ReferenceSkeleton
	* @param SkeletalMeshName - Name of the skeletal mesh
	* @param MaxBonesPerVert - Maximum number of bones per vert in the resulting LODModel.
	* @param SrcLODModel     - Reference model - provided in case some closest per-vertex data needs to be transfered (e.g. alternate weights) 
	* @param SkinnedMesh     - the source mesh
	* @param RefSkeleton    - reference skeleton
	* @param VertexAttributeInfos The vertex attribute information from the original and how it is packed in the skinned mesh.
	* @param NewModel       - resulting MeshLODModel
	*/
	void ConvertToFSkeletalMeshLODModel( const FString& SkeletalMeshName,
		                                 const int32 MaxBonesPerVert,
		                                 const FSkeletalMeshLODModel& SrcLODModel,
		                                 const SkeletalSimplifier::FSkinnedSkeletalMesh& SkinnedMesh,
		                                 const FReferenceSkeleton& RefSkeleton,
		                                 const TArray<FVertexAttributeInfo>& VertexAttributeInfos,
		                                 FSkeletalMeshLODModel& NewModel,
										 const bool bReducingSourceModel,
										 const ITargetPlatform* TargetPlatform) const;

	/**
	* Add the SourceModelInflunces to the LODModel in the case that alternate weights exists.
	* @param SrcLODModel       - reference LODModel that holds the Src WeightOverrideData
	* @param SkinnedMesh       - the source mesh, already used to generate the UpdatedModel
	* @param LODModel          - Updated MeshLODModel
	*/
	void AddSourceModelInfluences( const FSkeletalMeshLODModel& SrcLODModel,
	                               const SkeletalSimplifier::FSkinnedSkeletalMesh& SkinnedMesh,
		                           FSkeletalMeshLODModel& LODModel,
								   const FString& SkeletalMeshName) const;

	/**
	* Updates the alternate weights that correspond to the soft vertices
	* @param MaxBonesPerVert   - Limits the number of bones with non-zero weights on each vert.
	* @param LODModel          - Updated MeshLODModel
	*/
	void UpdateAlternateWeights(const int32 MaxBonesPerVertex, FSkeletalMeshLODModel& LODModel) const;

	/**
	* Simplify the mesh
	* @param Settings         - the settings that control the simplifier
	* @param Bounds           - the radius of the source mesh
	* @param InOutSkinnedMesh - the skinned mesh 
	*/
	float SimplifyMesh( const FSkeletalMeshOptimizationSettings& Settings,
		                const FBoxSphereBounds& Bounds,
		                SkeletalSimplifier::FSkinnedSkeletalMesh& InOutSkinnedMesh ) const ;


	/**
	* Extract data in SOA form needed for the MeshUtilities.BuildSkeletalMesh
	* to build the new skeletal mesh.
	* @param SkinnedMesh      -  input format for the mesh
	* @param SkeletalMeshData - struct of array format needed for the mesh utilities
	*/
	void ExtractFSkeletalData( const SkeletalSimplifier::FSkinnedSkeletalMesh& SkinnedMesh,
		                       FSkeletalMeshData& SkeletalMeshData ) const;

	/**
	* Compute the UVBounds for the each channel on the mesh
	* @param Mesh     - Input mesh
	* @param UVBounds - Resulting UV bounds
	*/
	void ComputeUVBounds( const SkeletalSimplifier::FSkinnedSkeletalMesh& Mesh,
		                  FVector2f(&UVBounds)[2 * SkeletalSimplifier::MeshVertType::BasicAttrContainerType::NumUVs] ) const;

	/**
	* Clamp the UVs on the mesh
	* @param UVBounds - per channel bounds for UVs
	* @param Mesh     - Mesh to update. 
	*/
	void ClampUVBounds( const FVector2f(&UVBounds)[2 * SkeletalSimplifier::MeshVertType::BasicAttrContainerType::NumUVs],
		                SkeletalSimplifier::FSkinnedSkeletalMesh& Mesh ) const;

	/**
	* Reduce the number of bones on the Mesh to a max number.
	* This will re-normalize the weights.
	* @param Mesh            - the mesh to update
	* @param MaxBonesPerVert - the max number of bones on each vert. 
	*/
	void TrimBonesPerVert( SkeletalSimplifier::FSkinnedSkeletalMesh& Mesh, int32 MaxBonesPerVert ) const ;

	/**
	* If a vertex has one of the prioritized bones as its' major bone, associated the PrioritizedFeatures.Weight
	* Likewise if the section ID for the vert is in a prioritized section, associate the prioritized weight 
	*/
	void UpdateSpecializedVertWeights(const FPrioritizedFeatures& PrioritizedFeatures, const TArray<int32>& VertToSectionId, SkeletalSimplifier::FSkinnedSkeletalMesh& SkinnedSkeletalMesh) const;


};

struct  FQuadricSkeletalMeshReduction::FSkeletalMeshData
{
	TArray<SkeletalMeshImportData::FVertInfluence> Influences;
	TArray<SkeletalMeshImportData::FMeshWedge> Wedges;
	TArray<SkeletalMeshImportData::FMeshFace> Faces;
	TArray<FVector3f> Points;
	uint32 TexCoordCount;
};


struct FQuadricSkeletalMeshReduction::FSectionRange
{
	int32 Begin;
	int32 End;
};

struct FQuadricSkeletalMeshReduction::FPrioritizedFeatures
{
	TArray<int32> BoneIds;
	TArray<int32> SectionIds;
	float Weight;

	bool HasFeatures() const
	{
		return (BoneIds.Num() >0 || SectionIds.Num() > 0);
	}
};

struct FQuadricSkeletalMeshReduction::FVertexAttributeInfo
{
	/* The name of the vertex attribute */
	FName AttributeName;
	
	/* Number of float components per vertex for this attribute */
	int32 ComponentCount;
	
	/* Offset into the reducers's compacted attribute storage for unpacking. */
	int32 Offset;
};

/**
*  Required MeshReduction Interface.
*/
class FSkeletalMeshReduction : public ISkeletalMeshReduction
{
public:
	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;


	/** IMeshReductionModule interface.*/
	virtual class IMeshReduction* GetSkeletalMeshReductionInterface() override
	{
		if (IsInGameThread())
		{
			//Load dependent modules in case the reduction is call later during a multi threaded call
			FModuleManager::Get().LoadModuleChecked<IMeshBoneReductionModule>("MeshBoneReduction");
			FModuleManager::Get().LoadModuleChecked<IMeshUtilities>("MeshUtilities");
		}
		return &SkeletalMeshReducer;
	}

	// not supported !
	virtual class IMeshReduction* GetStaticMeshReductionInterface()   override
	{
		return nullptr;
	}

	// not supported !
	virtual class IMeshMerging*   GetMeshMergingInterface()           override
	{
		return nullptr;
	}

	// not supported !
	virtual class IMeshMerging* GetDistributedMeshMergingInterface()  override
	{
		return nullptr;
	}


	virtual FString GetName() override
	{
		return FString("SkeletalMeshReduction");
	};

private:
	FQuadricSkeletalMeshReduction  SkeletalMeshReducer;
};

DEFINE_LOG_CATEGORY_STATIC(LogSkeletalMeshReduction, Log, All);

IMPLEMENT_MODULE(FSkeletalMeshReduction, SkeletalMeshReduction)


void FSkeletalMeshReduction::StartupModule()
{

	IModularFeatures::Get().RegisterModularFeature(IMeshReductionModule::GetModularFeatureName(), this);
}


void FSkeletalMeshReduction::ShutdownModule()
{

	IModularFeatures::Get().UnregisterModularFeature(IMeshReductionModule::GetModularFeatureName(), this);

}

bool FQuadricSkeletalMeshReduction::ReduceSkeletalMesh( USkeletalMesh* SkeletalMesh, int32 LODIndex, const ITargetPlatform* TargetPlatform)
{
	check(SkeletalMesh);
	check(LODIndex >= 0);
	check(LODIndex <= SkeletalMesh->GetLODNum());

	FSkeletalMeshModel* SkeletalMeshResource = SkeletalMesh->GetImportedModel();
	check(SkeletalMeshResource);
	check(LODIndex <= SkeletalMeshResource->LODModels.Num());

	FScopedSkeletalMeshPostEditChange ScopedPostEditChange(SkeletalMesh);
	ReduceSkeletalMesh(*SkeletalMesh, *SkeletalMeshResource, LODIndex, TargetPlatform);

	return true;
}

bool FQuadricSkeletalMeshReduction::RemoveMeshSection(FSkeletalMeshLODModel& Model, int32 SectionIndex) const 
{
	// Need a valid section
	if (!Model.Sections.IsValidIndex(SectionIndex))
	{
		return false;
	}

	FSkelMeshSection& SectionToRemove = Model.Sections[SectionIndex];

	if (SectionToRemove.CorrespondClothAssetIndex != INDEX_NONE)
	{
		// Can't remove this, clothing currently relies on it
		return false;
	}

	const uint32 NumVertsToRemove   = SectionToRemove.GetNumVertices();
	const uint32 BaseVertToRemove   = SectionToRemove.BaseVertexIndex;
	const uint32 NumIndicesToRemove = SectionToRemove.NumTriangles * 3;
	const uint32 BaseIndexToRemove  = SectionToRemove.BaseIndex;


	// Strip indices
	Model.IndexBuffer.RemoveAt(BaseIndexToRemove, NumIndicesToRemove);

	Model.Sections.RemoveAt(SectionIndex);

	// Fixup indices above base vert
	for (uint32& Index : Model.IndexBuffer)
	{
		if (Index >= BaseVertToRemove)
		{
			Index -= NumVertsToRemove;
		}
	}

	Model.NumVertices -= NumVertsToRemove;

	// Fixup anything needing section indices
	for (FSkelMeshSection& Section : Model.Sections)
	{
		// Push back clothing indices
		if (Section.CorrespondClothAssetIndex > SectionIndex)
		{
			Section.CorrespondClothAssetIndex--;
		}

		// Removed indices, re-base further sections
		if (Section.BaseIndex > BaseIndexToRemove)
		{
			Section.BaseIndex -= NumIndicesToRemove;
		}

		// Remove verts, re-base further sections
		if (Section.BaseVertexIndex > BaseVertToRemove)
		{
			Section.BaseVertexIndex -= NumVertsToRemove;
		}
	}
	return true;
}


void FQuadricSkeletalMeshReduction::ConvertToFSkinnedSkeletalMesh(
	const FSkeletalMeshLODModel& SrcLODModel,
	const TArray<FMatrix>& BoneMatrices,
	const int32 LODIndex,
	SkeletalSimplifier::FSkinnedSkeletalMesh& OutSkinnedMesh,
	const FString& SkeletalMeshName,
	TArray<FVertexAttributeInfo>& VertexAttributeInfos,
	TArray<int32>* VertToSectionMap
	) const
{



	auto ApplySkinning = [](const FMatrix& XForm, FSoftSkinVertex& Vertex)->bool
	{
		// Some imported models will have garbage tangent space
		bool bHasBadNTB =  ( Vertex.TangentX.ContainsNaN() || Vertex.TangentY.ContainsNaN() || Vertex.TangentZ.ContainsNaN() );

		// transform position
		FVector3f WeightedPosition = (FVector4f)XForm.TransformPosition((FVector)Vertex.Position);

		// transform tangent space
		FVector3f WeightedTangentX(1.f, 0.f, 0.f);
		FVector3f WeightedTangentY(0.f, 1.f, 0.f);
		FVector3f WeightedTangentZ(0.f, 0.f, 1.f);

		if (!bHasBadNTB)
		{
			WeightedTangentX = (FVector4f)XForm.TransformVector((FVector)Vertex.TangentX);
			WeightedTangentY = (FVector4f)XForm.TransformVector((FVector)Vertex.TangentY);
			WeightedTangentZ = (FVector4f)XForm.TransformVector((FVector4)Vertex.TangentZ);
		}
		
		Vertex.TangentX   = WeightedTangentX.GetSafeNormal();
		Vertex.TangentY   = WeightedTangentY.GetSafeNormal();
		float WComponent  = (bHasBadNTB) ? 1.f : Vertex.TangentZ.W;             
		Vertex.TangentZ   = WeightedTangentZ.GetSafeNormal();
		Vertex.TangentZ.W = WComponent;
		Vertex.Position   = WeightedPosition;

		return bHasBadNTB;
	};

	auto CreateSkinningMatrix = [&BoneMatrices](const FSoftSkinVertex& Vertex, const FSkelMeshSection& Section, bool& bValidBoneWeights)->FMatrix
	{
		// Compute the inverse of the total bone influence for this vertex.
		
		float InvTotalInfluence = UE::AnimationCore::InvMaxRawBoneWeightFloat;   // expected default - anything else could indicate a problem with the asset.
		{
			int32 TotalInfluence = 0;

			for (int32 i = 0; i < MAX_TOTAL_INFLUENCES; ++i)
			{
				const uint16 BoneInfluence = Vertex.InfluenceWeights[i];
				TotalInfluence += BoneInfluence;
			}

			if (TotalInfluence != UE::AnimationCore::MaxRawBoneWeight) // 255 is the expected value.  This logic just allows for graceful failure.
			{
				// Not expected value - record that.
				bValidBoneWeights = false;

				if (TotalInfluence == 0)
				{
					InvTotalInfluence = 0.f;
				}
				else
				{
					InvTotalInfluence = 1.f / float(TotalInfluence);
				}
			}
		}


		// Build the blended matrix 

		FMatrix BlendedMatrix(ForceInitToZero);

		int32 ValidInfluenceCount = 0;
		
		const TArray<uint16>& BoneMap = Section.BoneMap;

		for (int32 i = 0; i < MAX_TOTAL_INFLUENCES; ++i)
		{
			const uint16 BoneIndex    = Vertex.InfluenceBones[i];
			const uint16 BoneInfluence = Vertex.InfluenceWeights[i];

			// Accumulate the bone influence for this vert into the BlendedMatrix

			if (BoneInfluence > 0)
			{
				check(BoneIndex < BoneMap.Num());
				const uint16 SectionBoneId = BoneMap[BoneIndex]; // Third-party tool uses an additional indirection bode table here 
				const float  BoneWeight = BoneInfluence * InvTotalInfluence;  // convert to [0,1] float

				if (BoneMatrices.IsValidIndex(SectionBoneId))
				{
					ValidInfluenceCount++;
					const FMatrix BoneMatrix = BoneMatrices[SectionBoneId];
					BlendedMatrix += (BoneMatrix * BoneWeight);
				}
			}
		}

		// default identity matrix for the special case of the vertex having no valid transforms..

		if (ValidInfluenceCount == 0)
		{
			BlendedMatrix = FMatrix::Identity;
		}

		

		return BlendedMatrix;
	};



	// Copy the vertices into a single buffer

	TArray<FSoftSkinVertex> SoftSkinVertices;
	SrcLODModel.GetVertices(SoftSkinVertices);
	const int32 SectionCount = SrcLODModel.Sections.Num();

	// functor: true if this section should be included.

	auto SkipSection = [&SrcLODModel, LODIndex](int32 SectionIndex)->bool
	{
		if (SrcLODModel.Sections[SectionIndex].bDisabled)
		{
			return true;
		}
		int32 MaxLODIndex = SrcLODModel.Sections[SectionIndex].GenerateUpToLodIndex;
		return (MaxLODIndex != -1 && MaxLODIndex < LODIndex);
	};

	// Fill in the attribute infos
	VertexAttributeInfos.Reset();
	int32 AttributeOffset = 0;
	for (const TPair<FName, FSkeletalMeshModelVertexAttribute>& Attribute: SrcLODModel.VertexAttributes)
	{
		VertexAttributeInfos.Add({Attribute.Key, Attribute.Value.ComponentCount, AttributeOffset});
		AttributeOffset += Attribute.Value.ComponentCount; 
	}

	// Count the total number of verts, but only the number of triangles that
	// are used in sections we don't skip.
	// NB: This could result zero triangles, but a non-zero number of verts.
	//     i.e. we aren't going to try to compact the vertex array.

	TArray<FSectionRange> SectionRangeArray; // Keep track of the begin / end vertex in this section

	int32 VertexCount = 0;
	
	for (int32 SectionIndex = 0; SectionIndex < SectionCount; ++SectionIndex)
	{
		const FSkelMeshSection& Section = SrcLODModel.Sections[SectionIndex];
		FSectionRange SectionRange;
		SectionRange.Begin = VertexCount;
		SectionRange.End   = VertexCount + Section.SoftVertices.Num();

		SectionRangeArray.Add(SectionRange);

		VertexCount = SectionRange.End;
	}

	// Verify that the model has an allowed number of textures
	const uint32 TexCoordCount = SrcLODModel.NumTexCoords;
	check(TexCoordCount <= MAX_TEXCOORDS);

	
	// Update the verts to the skinned location.
	int32 NumBadNTBSpace = 0;
	for (int32 SectionIndex = 0; SectionIndex < SectionCount; ++SectionIndex)
	{
		const FSkelMeshSection& Section  = SrcLODModel.Sections[SectionIndex];
		const FSectionRange VertexRange  = SectionRangeArray[SectionIndex];
		
		// Loop over the vertices in this section.
		bool bHasValidBoneWeights = true;
		for (int32 VertexIndex = VertexRange.Begin; VertexIndex < VertexRange.End; ++VertexIndex)
		{
			FSoftSkinVertex& SkinVertex = SoftSkinVertices[VertexIndex];

			// Use the bone weights for this vertex to create a blended matrix 
			const FMatrix BlendedMatrix = CreateSkinningMatrix(SkinVertex, Section, bHasValidBoneWeights);


			// Update this Skin Vertex to the correct location, normal, etc.
			// also replace NaN tangent spaces with default tangent before skinning
			bool bHasBadNTB = ApplySkinning(BlendedMatrix, SkinVertex);

			if (bHasBadNTB)
			{
				NumBadNTBSpace++;
			}
		}

		// Report any error with invalid bone weights
		if (!bHasValidBoneWeights && !SkipSection(SectionIndex))
		{
			UE_LOGF(LogSkeletalMeshReduction, Warning, "Building LOD %d - Encountered non normalized vertex weights in source. %ls", LODIndex, *SkeletalMeshName);
		}
	}

	if (NumBadNTBSpace > 0)
	{
		UE_LOGF(LogSkeletalMeshReduction, Warning, "There were NaNs in the Tangent Space of %d source model vertices. %ls", NumBadNTBSpace, *SkeletalMeshName);
	}

	// -- Make the index buffer; skipping the "SkipSections"

	// How many triangles?

	int32 NumTriangles = 0;
	for (int32 s = 0; s < SectionCount; ++s)
	{
		if (SkipSection(s))
		{
			continue;
		}
		NumTriangles += SrcLODModel.Sections[s].NumTriangles;
	}

	const int32 NumIndices = NumTriangles * 3;

	OutSkinnedMesh.Resize(NumTriangles, VertexCount);
	OutSkinnedMesh.SetTexCoordCount(TexCoordCount);

	// local names
	uint32* OutIndexBuffer                            = OutSkinnedMesh.IndexBuffer;
	SkeletalSimplifier::MeshVertType* OutVertexBuffer = OutSkinnedMesh.VertexBuffer;

	// Construct the index buffer

	{
		int32 tmpId = 0;
		for (int32 s = 0; s < SectionCount; ++s)
		{
			if (SkipSection(s))
			{
				continue;
			}
			const auto& SrcIndexBuffer = SrcLODModel.IndexBuffer;

			const FSkelMeshSection& Section = SrcLODModel.Sections[s];

			const uint32 FirstIndex = Section.BaseIndex;
			const uint32 LastIndex = FirstIndex + Section.NumTriangles * 3;

			for (uint32 i = FirstIndex; i < LastIndex; ++i)
			{
				const uint32 vertexId = SrcIndexBuffer[i];
				OutIndexBuffer[tmpId] = (int32)vertexId;
				tmpId++;
			}
		}
	}
	
	// Copy all the verts over.  NB: We don't skip any sections 
	// so the index buffer offsets will still be valid.
	// NB: we do clamp the UVs to +/- 1024

	for (int32 SectionIndex = 0; SectionIndex < SectionCount; ++SectionIndex)
	{
		const FSkelMeshSection& Section = SrcLODModel.Sections[SectionIndex];
		const auto& BoneMap = Section.BoneMap;

		const FSectionRange VertexRange = SectionRangeArray[SectionIndex];

		for (int32 v = VertexRange.Begin; v < VertexRange.End; ++v)
		{
			const auto& SkinnedVertex = SoftSkinVertices[v];

			auto& BasicAttrs  = OutVertexBuffer[v].BasicAttributes;
			auto& SparseBones = OutVertexBuffer[v].SparseBones;

			BasicAttrs.Normal    = SkinnedVertex.TangentZ;
			BasicAttrs.Tangent   = SkinnedVertex.TangentX;
			BasicAttrs.BiTangent = SkinnedVertex.TangentY;

			for (uint32 t = 0; t < TexCoordCount; ++t)
			{
				BasicAttrs.TexCoords[t].X = FMath::Clamp(SkinnedVertex.UVs[t].X, -1024.f, 1024.f);
				BasicAttrs.TexCoords[t].Y = FMath::Clamp(SkinnedVertex.UVs[t].Y, -1024.f, 1024.f);
			}
			for (uint32 t = TexCoordCount; t < MAX_TEXCOORDS; ++t)
			{
				BasicAttrs.TexCoords[t].X = 0.f;
				BasicAttrs.TexCoords[t].Y = 0.f;
			}

			BasicAttrs.Color = SkinnedVertex.Color;
			OutVertexBuffer[v].ClosestSrcVertIndex = v; // index of the closest vert w.r.t  SrcLODModel.GetVertices(SoftSkinVertices);
			OutVertexBuffer[v].MaterialIndex   = 0; // default to be over-written
			OutVertexBuffer[v].Position = SkinnedVertex.Position;

			for (int32 i = 0; i < MAX_TOTAL_INFLUENCES; ++i)
			{
				int32 localBoneId = (int32)SkinnedVertex.InfluenceBones[i];
				const uint16 boneId = BoneMap[localBoneId];

				const uint16 Influence = SkinnedVertex.InfluenceWeights[i];
				double boneWeight = ((double)Influence) / UE::AnimationCore::MaxRawBoneWeightFloat;

				// For right now, only store bone weights that are greater than zero,
				// by default the sparse data structure assumes a value of zero for 
				// any non-initialized bones.

				if (Influence > 0)
				{
					SparseBones.SetElement((int32)boneId, boneWeight);
				}
			}
		}

		// Copy over the attributes. 
		for (const FVertexAttributeInfo& AttributeInfo: VertexAttributeInfos)
		{
			const FSkeletalMeshModelVertexAttribute& VertexAttribute = SrcLODModel.VertexAttributes[AttributeInfo.AttributeName]; 			
			
			for (int32 Vertex = VertexRange.Begin; Vertex < VertexRange.End; ++Vertex)
			{
				auto& VertexAttrs  = OutVertexBuffer[Vertex].AdditionalAttributes;
				const float* AttributeData = VertexAttribute.Values.GetData() + Vertex * VertexAttribute.ComponentCount; 
				
				for (int32 Component = 0; Component < VertexAttribute.ComponentCount; Component++)
				{
					VertexAttrs.SetElement(AttributeInfo.Offset + Component, AttributeData[Component]); 
				}
			}
		}
	}

	// store sectionID or MaterialID in the material id (there is a one to one mapping between them).

	for (int32 s = 0; s < SectionCount; ++s)
	{
		if (SkipSection(s))
		{
			continue;
		}
		uint16 MaterialId = SrcLODModel.Sections[s].MaterialIndex;

		const FSectionRange VertexRange = SectionRangeArray[s];

		for (int32 v = VertexRange.Begin; v < VertexRange.End; ++v)
		{
			//OutVertexBuffer[v].MaterialIndex = s;
			OutVertexBuffer[v].MaterialIndex = MaterialId;
		}
	}

	// if requested, make a map of the vertex to original section id
	if (VertToSectionMap != nullptr)
	{
		const int32 InvalidSectionId = -1;
		VertToSectionMap->Empty();
		VertToSectionMap->Init(InvalidSectionId, VertexCount);
	
		TArray<int32>& ToSection = *VertToSectionMap;
		for (int32 s = 0; s < SectionCount; ++s)
		{
			if (SkipSection(s))
			{
				continue;
			}
			int32 OriginalDataSectionIndex = SrcLODModel.Sections[s].OriginalDataSectionIndex;

			const FSectionRange VertexRange = SectionRangeArray[s];

			for (int32 v = VertexRange.Begin; v < VertexRange.End; ++v)
			{
				ToSection[v] = OriginalDataSectionIndex;
			}
		}
	}
	// Put the vertex in a "correct" state.
	//    "corrects" normals (ensures that they are orthonormal)
	//    re-orders the bones by weight (highest to lowest)

	for (int32 s = 0; s < SectionCount; ++s)
	{
		if (SkipSection(s))
		{
			continue;
		}

		const FSectionRange VertexRange = SectionRangeArray[s];

		for (int32 v = VertexRange.Begin; v < VertexRange.End; ++v)
		{
			OutVertexBuffer[v].Correct();
		}
	}

	// Compact the mesh to remove any unreferenced verts
	// and fix-up the index buffer
	TArray<int32> CompactVertMap; 
	OutSkinnedMesh.Compact(&CompactVertMap);

	// Remap the VertToSectionMap to use the post-compact vert ids.
	if (VertToSectionMap != nullptr)
	{
			const int32 CompactVertCount = OutSkinnedMesh.NumVertices();
			TArray<int32> TmpVertToSectionMap;
			const int32 InvalidSectionId = -1;
			TmpVertToSectionMap.Empty();
			TmpVertToSectionMap.Init(InvalidSectionId, CompactVertCount);
			checkSlow(CompactVertCount == CompactVertMap.Num());
			for (int32 DstID = 0; DstID < CompactVertMap.Num(); DstID++)
			{
				const int32 SrcID = CompactVertMap[DstID];
				TmpVertToSectionMap[DstID] = (*VertToSectionMap)[SrcID];
			}

			Swap(TmpVertToSectionMap, *VertToSectionMap);
	}
}


void FQuadricSkeletalMeshReduction::UpdateSpecializedVertWeights(const FPrioritizedFeatures& PrioritizedFeatures, const TArray<int32>& VertToOriginalSectionId, SkeletalSimplifier::FSkinnedSkeletalMesh& SkinnedSkeletalMesh) const
{

	if (!PrioritizedFeatures.HasFeatures())
	{
		return;
	}
	
	const float Weight = PrioritizedFeatures.Weight;

	int32 NumVerts = SkinnedSkeletalMesh.NumVertices();
	check(VertToOriginalSectionId.Num() == NumVerts);

	const int32 InvalidSectionId = -1;
	//If a vertex has one of the important bones as its' major bone, associated the ImportantBones.Weight
	for (int32 i = 0; i < NumVerts; ++i)
	{
		const int32 OriginalSectionId = VertToOriginalSectionId[i];
		auto& Vert = SkinnedSkeletalMesh.VertexBuffer[i];
		Vert.SpecializedWeight = 0.f;

		// Test if this vert is associated with a priority bone
		const auto& Bones = Vert.GetSparseBones();
		if (!Bones.bIsEmpty())
		{
			auto CIter = Bones.GetData().CreateConstIterator();

			const int32 FirstBone = CIter.Key(); // Bones are ordered by descending weight

			if (PrioritizedFeatures.BoneIds.Contains(FirstBone))
			{
				Vert.SpecializedWeight = Weight;
			}
		}
		
		// Test if this vert is associated with a priority section
		if (OriginalSectionId != InvalidSectionId && PrioritizedFeatures.SectionIds.Contains(OriginalSectionId))
		{
			//Using += here will make bone and section prioritized verts stronger than only bone or only section prioritized verts
			Vert.SpecializedWeight += Weight;
		}

	}
}



void FQuadricSkeletalMeshReduction::TrimBonesPerVert( SkeletalSimplifier::FSkinnedSkeletalMesh& Mesh, int32 MaxBonesPerVert ) const
{
	// Loop over all the verts in the mesh, and reduce the bone count.

	SkeletalSimplifier::MeshVertType* VertexBuffer = Mesh.VertexBuffer;

	for (int32 i = 0, I = Mesh.NumVertices(); i < I; ++i)
	{
		SkeletalSimplifier::MeshVertType& Vertex = VertexBuffer[i];

		auto& SparseBones = Vertex.SparseBones;
		SparseBones.Correct(MaxBonesPerVert);
	}

}


void FQuadricSkeletalMeshReduction::ComputeUVBounds( const SkeletalSimplifier::FSkinnedSkeletalMesh& Mesh,
	                                                 FVector2f(&UVBounds)[2 * SkeletalSimplifier::MeshVertType::BasicAttrContainerType::NumUVs] ) const
{
	// Zero the bounds
	{
		const int32 NumUVs = SkeletalSimplifier::MeshVertType::BasicAttrContainerType::NumUVs;

		for (int32 i = 0; i < 2 * NumUVs; ++i)
		{
			UVBounds[i] = FVector2f(ForceInitToZero);
		}
	}


	{
		const int32 NumValidUVs = Mesh.TexCoordCount();
		for (int32 i = 0; i < NumValidUVs; ++i)
		{
			UVBounds[2 * i] = FVector2f(FLT_MAX, FLT_MAX);
			UVBounds[2 * i + 1] = FVector2f(-FLT_MAX, -FLT_MAX);
		}

		for (int32 i = 0; i < Mesh.NumVertices(); ++i)
		{
			const auto& Attrs = Mesh.VertexBuffer[i].BasicAttributes;
			for (int32 t = 0; t < NumValidUVs; ++t)
			{
				UVBounds[2 * t].X = FMath::Min(Attrs.TexCoords[t].X, UVBounds[2 * t].X);
				UVBounds[2 * t].Y = FMath::Min(Attrs.TexCoords[t].Y, UVBounds[2 * t].Y);

				UVBounds[2 * t + 1].X = FMath::Max(Attrs.TexCoords[t].X, UVBounds[2 * t + 1].X);
				UVBounds[2 * t + 1].Y = FMath::Max(Attrs.TexCoords[t].Y, UVBounds[2 * t + 1].Y);
			}
		}
	}
}

void FQuadricSkeletalMeshReduction::ClampUVBounds( const FVector2f(&UVBounds)[2 * SkeletalSimplifier::MeshVertType::BasicAttrContainerType::NumUVs],
	                                               SkeletalSimplifier::FSkinnedSkeletalMesh& Mesh ) const
{
	const int32 NumValidUVs = Mesh.TexCoordCount();


	for (int32 i = 0; i < Mesh.NumVertices(); ++i)
	{
		auto& Attrs = Mesh.VertexBuffer[i].BasicAttributes;
		for (int32 t = 0; t < NumValidUVs; ++t)
		{
			Attrs.TexCoords[t].X = FMath::Clamp(Attrs.TexCoords[t].X, UVBounds[2 * t].X, UVBounds[2 * t + 1].X);
			Attrs.TexCoords[t].Y = FMath::Clamp(Attrs.TexCoords[t].Y, UVBounds[2 * t].Y, UVBounds[2 * t + 1].Y);
		}
	}
}


float FQuadricSkeletalMeshReduction::SimplifyMesh( const FSkeletalMeshOptimizationSettings& Settings,
	                                               const FBoxSphereBounds& Bounds,
	                                               SkeletalSimplifier::FSkinnedSkeletalMesh& Mesh
                                                  ) const
{

	const bool bUseLegacyAttrGrad = Settings.bImproveTrianglesForCloth;
	// Convert settings to weights and a termination criteria

	// Determine the stop criteria used

	const bool bUseVertexPercentCriterion = Settings.TerminationCriterion == SkeletalMeshTerminationCriterion::SMTC_NumOfVerts || Settings.TerminationCriterion == SkeletalMeshTerminationCriterion::SMTC_TriangleOrVert;
	const bool bUseTrianglePercentCriterion = Settings.TerminationCriterion == SkeletalMeshTerminationCriterion::SMTC_NumOfTriangles || Settings.TerminationCriterion == SkeletalMeshTerminationCriterion::SMTC_TriangleOrVert;

	const bool bUseMaxVertNumCriterion = Settings.TerminationCriterion == SkeletalMeshTerminationCriterion::SMTC_AbsNumOfVerts || Settings.TerminationCriterion == SkeletalMeshTerminationCriterion::SMTC_AbsTriangleOrVert;
	const bool bUseMaxTrisNumCriterion = Settings.TerminationCriterion == SkeletalMeshTerminationCriterion::SMTC_AbsNumOfTriangles || Settings.TerminationCriterion == SkeletalMeshTerminationCriterion::SMTC_AbsTriangleOrVert;

	// We can support a stopping criteria based on the MaxDistance the new vertex is from the plans of the source triangles.
	// but there seems to be no good use for this.  We are better off just using triangle count.
	const float MaxDist = FLT_MAX; // (Settings.ReductionMethod != SkeletalMeshOptimizationType::SMOT_NumOfTriangles) ? Settings.MaxDeviationPercentage * Bounds.SphereRadius : FLT_MAX;
	const uint32 SrcTriNum = Mesh.NumIndices() / 3;
	const float TriangleRetainRatio = FMath::Clamp(Settings.NumOfTrianglesPercentage, 0.f, 1.f);
	uint32 TargetTriNum = (bUseTrianglePercentCriterion) ? FMath::CeilToInt(TriangleRetainRatio * SrcTriNum) : Settings.MaxNumOfTriangles;
	if (Settings.TerminationCriterion == SkeletalMeshTerminationCriterion::SMTC_NumOfTriangles)
	{
		TargetTriNum = FMath::Min(TargetTriNum, Settings.MaxNumOfTrianglesPercentage);
	}

	const uint32 MinTriNumToRetain = (bUseTrianglePercentCriterion || bUseMaxTrisNumCriterion) ? FMath::Max((uint32)4, TargetTriNum) : 4;
	const float MaxCollapseCost = FLT_MAX;

	const uint32 SrcVertNum = Mesh.NumVertices();
	const float VertRetainRatio = FMath::Clamp(Settings.NumOfVertPercentage, 0.f, 1.f);
	uint32 TargetVertNum = (bUseVertexPercentCriterion) ? FMath::CeilToInt(VertRetainRatio * SrcVertNum) : Settings.MaxNumOfVerts + 1;
	if (Settings.TerminationCriterion == SkeletalMeshTerminationCriterion::SMTC_NumOfVerts)
	{
		uint32 Delta = Settings.MaxNumOfVertsPercentage < MAX_uint32 ? 1 : 0;
		TargetVertNum = FMath::Min(TargetVertNum, Settings.MaxNumOfVertsPercentage + Delta);
	}

	const uint32 MinVerNumToRetain = (bUseVertexPercentCriterion || bUseMaxVertNumCriterion) ? FMath::Max((uint32)6, TargetVertNum) : 6;

	const float VolumeImportance      = FMath::Clamp(Settings.VolumeImportance, 0.f, 2.f);
	const bool bLockEdges             = Settings.bLockEdges;
	const bool bPreserveVolume        = (VolumeImportance > 1.e-4);
	const bool bEnforceBoneBoundaries = Settings.bEnforceBoneBoundaries;
	const bool bMergeCoincidentVertBones = Settings.bMergeCoincidentVertBones;
	const bool bLockColorBoundaries   = Settings.bLockColorBounaries;

	// Terminator tells the simplifier when to stop
	SkeletalSimplifier::FSimplifierTerminator Terminator(MinTriNumToRetain, SrcTriNum, MinVerNumToRetain, SrcVertNum, MaxCollapseCost, MaxDist);

	double NormalWeight    =16.00;
	double TangentWeight   = 0.10;
	double BiTangentWeight = 0.10;
	double UVWeight        = 0.50;
	double BoneWeight      = 0.25;
	double ColorWeight     = 0.10;
	/**
	// Magic table used to scale the default simplifier weights.
	// Numbers like these were used to express controls for the 3rd party tool
	const float ImportanceTable[] =
	{
		0.0f,	// OFF
		0.125f,	// Lowest
		0.35f,	// Low,
		1.0f,	// Normal
		2.8f,	// High
		8.0f,	// Highest
	};
	static_assert(UE_ARRAY_COUNT(ImportanceTable) == SMOI_MAX, "Bad importance table size.");

	NormalWeight    *= ImportanceTable[Settings.ShadingImportance];
	TangentWeight   *= ImportanceTable[Settings.ShadingImportance];
	BiTangentWeight *= ImportanceTable[Settings.ShadingImportance];
	UVWeight        *= ImportanceTable[Settings.TextureImportance];
	BoneWeight      *= ImportanceTable[Settings.SkinningImportance];

	*/

	// Number of UV coords allocated.
	const int32 NumUVs = SkeletalSimplifier::MeshVertType::BasicAttrContainerType::NumUVs;

	FVector2f  UVBounds[2 * SkeletalSimplifier::MeshVertType::BasicAttrContainerType::NumUVs];
	ComputeUVBounds(Mesh, UVBounds);
	
	// Set up weights for the Basic Attributes (e.g. not the bones)
	SkeletalSimplifier::FMeshSimplifier::WeightArrayType  BasicAttrWeights; // by default constructs to zeros.
	{

		// Normal
		BasicAttrWeights[0] = NormalWeight;
		BasicAttrWeights[1] = NormalWeight;
		BasicAttrWeights[2] = NormalWeight;

		//Tangent
		BasicAttrWeights[3] = TangentWeight;
		BasicAttrWeights[4] = TangentWeight;
		BasicAttrWeights[5] = TangentWeight;
		//BiTangent
		BasicAttrWeights[6] = BiTangentWeight;
		BasicAttrWeights[7] = BiTangentWeight;
		BasicAttrWeights[8] = BiTangentWeight;

		//Color
		BasicAttrWeights[ 9] = ColorWeight; // r
		BasicAttrWeights[10] = ColorWeight; // b
		BasicAttrWeights[11] = ColorWeight; // b
		BasicAttrWeights[12] = ColorWeight; // alpha


		const int32 NumNonUVAttrs = 13;
		checkSlow(NumNonUVAttrs + NumUVs * 2 == BasicAttrWeights.Num());

		// Number of UVs actually used.
		const int32 NumValidUVs = Mesh.TexCoordCount();
		for (int32 i = 0; i < NumValidUVs; ++i)
		{
			FVector2f&  UVMin = UVBounds[2 * i];
			FVector2f&  UVMax = UVBounds[2 * i + 1];

			double URange = UVMax.X - UVMin.X;
			double VRange = UVMax.Y - UVMin.Y;

			double UWeight = (FMath::Abs(URange) > 1.e-5) ? UVWeight / URange : 0.;
			double VWeight = (FMath::Abs(VRange) > 1.e-5) ? UVWeight / VRange : 0.;

			BasicAttrWeights[NumNonUVAttrs + 2 * i    ] = UWeight; // U
			BasicAttrWeights[NumNonUVAttrs + 2 * i + 1] = VWeight; // V
		}

		for (int32 i = NumNonUVAttrs; i < NumNonUVAttrs + NumValidUVs * 2; ++i)
		{
			BasicAttrWeights[i] = UVWeight; // 0.5;
		}

		for (int32 i = NumNonUVAttrs + NumValidUVs * 2; i < NumNonUVAttrs + NumUVs * 2; ++i)
		{
			BasicAttrWeights[i] = 0.;
		}

	}

	// Additional parameters

	const bool bWeldVertexColorAttributes = true;
	const float EdgeWeightValue = 128.f;

	const float CoAlignmentLimit = FMath::Cos(45.f * PI / 180.); // 45 degrees limit

	// Create the simplifier
	
	SkeletalSimplifier::FMeshSimplifier  Simplifier(Mesh.VertexBuffer, (uint32)Mesh.NumVertices(),
		                                            Mesh.IndexBuffer, (uint32)Mesh.NumIndices(), 
		                                            CoAlignmentLimit, VolumeImportance, bPreserveVolume,  
													bEnforceBoneBoundaries, bMergeCoincidentVertBones,
													bUseLegacyAttrGrad);

	// The simplifier made a deep copy of the mesh.  

	Mesh.Empty();

	float PercentNMEdges = FMath::FloorToInt(10000.f * Simplifier.FractionNonManifoldEdges())/100.f;
	FString PrettyPercentNMEdges = FString::SanitizeFloat(PercentNMEdges);

	if (PercentNMEdges != 0.f)
	{ 
		UE_LOGF(LogSkeletalMeshReduction, Log, "%ls-percent of edges are shared by more than two triangles, these may cause problems.",  *PrettyPercentNMEdges);
	}
	// Add additional control parameters to the simplifier.

	{
		// Set the edge weight that tries to keep UVseams from splitting.

		Simplifier.SetBoundaryConstraintWeight(EdgeWeightValue);

		// Set the weights for the dense attributes.

		Simplifier.SetAttributeWeights(BasicAttrWeights);

		// Set the bone weight.

		SkeletalSimplifier::FMeshSimplifier::SparseWeightContainerType BoneWeights(BoneWeight);
		Simplifier.SetSparseAttributeWeights(BoneWeights);


		if (bLockEdges)
		{
			// If locking the boundary, this has be be done before costs are computed.
			Simplifier.SetBoundaryLocked();
		}

		if (bLockColorBoundaries)
		{
			// Lock the verts in edges that connect differnt colors.  Also locks verts that have multiple colors.
			Simplifier.SetColorEdgeLocked();
		}

	}

	// Do the actual simplification

	const float ResultError = Simplifier.SimplifyMesh(Terminator);

	// Resize the Mesh to hold the simplified result. Note the NumVerts might include some duplicates.

	Mesh.Resize(Simplifier.GetNumTris(), Simplifier.GetNumVerts());

	// Copy the simplified mesh back into Mesh

	Simplifier.OutputMesh(Mesh.VertexBuffer, Mesh.IndexBuffer, bWeldVertexColorAttributes, nullptr);

	// There might have some unused verts at the end of the vertex buffer that were generated by the possible duplicates

	Mesh.Compact();

	return ResultError;
}


void FQuadricSkeletalMeshReduction::ExtractFSkeletalData(const SkeletalSimplifier::FSkinnedSkeletalMesh& SkinnedMesh, FSkeletalMeshData& MeshData) const
{

	MeshData.TexCoordCount = SkinnedMesh.TexCoordCount();

	const int32 NumVerts   = SkinnedMesh.NumVertices();
	const int32 NumIndices = SkinnedMesh.NumIndices();
	const int32 NumTris    = NumIndices / 3;

	// Resize the MeshData.
	MeshData.Points.AddZeroed(NumVerts);
	MeshData.Faces.AddZeroed(NumTris);
	MeshData.Wedges.AddZeroed(NumIndices);

	TArray<FVector3f> PointNormals;
	TArray<uint32> PointList;
	TArray<uint32> PointInfluenceMap;  // index into MeshData.Influences.   Id = PointInfluenceMap[v];  first_influence_for_vert 'v' = MeshData.Influences[Id] 

	PointNormals.AddZeroed(NumVerts);

	PointList.Reserve(NumVerts);
	for (int32 i = 0; i < NumVerts; ++i)
	{
		PointList.Add(INDEX_NONE);
	}

	PointInfluenceMap.Reserve(NumVerts);
	for (int32 i = 0; i < NumVerts; ++i)
	{
		PointInfluenceMap.Add(INDEX_NONE);
	}

	
	// Per-vertex data
	
	for (uint32 v = 0; v < (uint32)NumVerts; ++v)
	{
		// The simplifier mesh vertex, has all the vertex attributes.

		const auto& SimpVertex = SkinnedMesh.VertexBuffer[v];
		
		// Copy location.
		
		MeshData.Points[v] = SimpVertex.GetPos();

		// Sort out the bones for this vert.

		PointInfluenceMap[v] = (uint32)MeshData.Influences.Num();

		// loop over the bones for this vertex, making weights.

		const auto& SparseBones = SimpVertex.GetSparseBones().GetData();

		int32 NumBonesAdded = 0;
		for (const auto& BoneData : SparseBones)
		{
			if (BoneData.Value > 0.)
			{
				SkeletalMeshImportData::FVertInfluence VertInfluence = { (float) BoneData.Value, v, (uint16) BoneData.Key};

				MeshData.Influences.Add(VertInfluence);
				NumBonesAdded++;
			}
		}
		
		// If no influences were added, add a default bone
		
		if (NumBonesAdded == 0)
		{
			SkeletalMeshImportData::FVertInfluence VertInfluence = { 0.f, v, (uint16)0 };

			MeshData.Influences.Add(VertInfluence);
	
		}
	}

	// loop over triangles.
	for (int32 t = 0; t < NumTris; ++t)
	{
		SkeletalMeshImportData::FMeshFace& Face = MeshData.Faces[t];

		
		uint32 MatId[3];

		// loop over the three corners for the triangle.
		// NB: We may have already visited these verts before..
		
		for (uint32 c = 0; c < 3; ++c)
		{
			const uint32 wedgeId = t * 3 + c;
			const uint32 vertId = SkinnedMesh.IndexBuffer[wedgeId];
			const auto& SimpVertex = SkinnedMesh.VertexBuffer[vertId];

			FVector3f WedgeNormal = SimpVertex.BasicAttributes.Normal;
			WedgeNormal.Normalize();

			Face.TangentX[c] = SimpVertex.BasicAttributes.Tangent;
			Face.TangentY[c] = SimpVertex.BasicAttributes.BiTangent;
			Face.TangentZ[c] = WedgeNormal;

			Face.iWedge[c] = wedgeId;

			MatId[c] = SimpVertex.MaterialIndex;

			//
			uint32 tmpVertId = vertId;
			FVector3f PointNormal = PointNormals[tmpVertId];

			if (PointNormal.SizeSquared() < KINDA_SMALL_NUMBER) // the array starts with 0'd out normals
			{
				PointNormals[tmpVertId] = WedgeNormal;
			}
			else // we have already visited this vert ..
			{
				while (FVector3f::DotProduct(PointNormal, WedgeNormal) - 1.f < -KINDA_SMALL_NUMBER)
				{
					tmpVertId = PointList[tmpVertId];
					if (tmpVertId == INDEX_NONE)
					{
						break;
					}
					checkSlow(tmpVertId < (uint32)PointList.Num());
					PointNormal = PointNormals[tmpVertId];
				}

				if (tmpVertId == INDEX_NONE)
				{
					// Add a copy of this point.. 
					FVector3f Point = MeshData.Points[vertId];
					tmpVertId = MeshData.Points.Add(Point);

					PointNormals.Add(WedgeNormal);

					uint32 nextVertId = PointList[vertId];
					PointList[vertId] = tmpVertId;
					PointList.Add(nextVertId);
					PointInfluenceMap.Add((uint32)MeshData.Influences.Num());

					int32 influenceId = PointInfluenceMap[vertId];
					while (MeshData.Influences[influenceId].VertIndex == vertId)
					{
						const auto& Influence = MeshData.Influences[influenceId];

						SkeletalMeshImportData::FVertInfluence VertInfluence = { Influence.Weight, tmpVertId, Influence.BoneIndex };
						MeshData.Influences.Add(VertInfluence);
						influenceId++;
					}
				}

			}

			// Populate the corresponding wedge.
			SkeletalMeshImportData::FMeshWedge& Wedge = MeshData.Wedges[wedgeId];
			Wedge.iVertex  = tmpVertId; // vertId;
			Wedge.Color    = SimpVertex.BasicAttributes.Color.ToFColor(true/** sRGB**/);
			for (int32 tcIdx = 0; tcIdx < MAX_TEXCOORDS; ++tcIdx)
			{
				Wedge.UVs[tcIdx] = SimpVertex.BasicAttributes.TexCoords[tcIdx];
			}

			

		}
		// The material id is only being stored on a per-vertex case..
		// but should be shared by all 3 verts in a triangle.

		Face.MeshMaterialIndex = (uint16)MatId[0];

	}

}

namespace
{
	struct FIntBoneFloatWeight
	{
		FIntBoneFloatWeight(float w, int32 b) : Weight(w), BoneId(b) {};

		float Weight;
		int32 BoneId;
	};


	/** Utility for use instead of SkeletalMeshLODModel::GetSectionFromVertexIndex() since we are going to visit every vertex */
	void CreateVertexToSectionMap(const FSkeletalMeshLODModel& LODModel, TArray<int32>& VertIdxToSectionMap)
	{
		// Create a map between the VertexID and the Section
		VertIdxToSectionMap.Empty(LODModel.NumVertices);
		VertIdxToSectionMap.AddUninitialized(LODModel.NumVertices);
		{
			int32 offset = 0;
			for (int32 sectionIdx = 0; sectionIdx < LODModel.Sections.Num(); ++sectionIdx)
			{
				const auto& Section = LODModel.Sections[sectionIdx];
				for (int32 i = 0; i < Section.NumVertices; ++i)
				{
					VertIdxToSectionMap[i + offset] = sectionIdx;
				}
				offset += Section.NumVertices;
			}
		}
	}


	void ZeroFRawSkinWeight(FRawSkinWeight& SkinWeight)
	{
		for (int b = 0; b < MAX_TOTAL_INFLUENCES; ++b)
		{
			SkinWeight.InfluenceBones[b] = 0;
		}
		for (int b = 0; b < MAX_TOTAL_INFLUENCES; ++b)
		{
			SkinWeight.InfluenceWeights[b] = 0;
		}
	}


	void Empty(FSkeletalMeshLODModel& LODModel)
	{
		LODModel.Empty();
	}

}

void  FQuadricSkeletalMeshReduction::AddSourceModelInfluences( const FSkeletalMeshLODModel& SrcLODModel,
		                                                       const SkeletalSimplifier::FSkinnedSkeletalMesh& SkinnedMesh, 
		                                                       FSkeletalMeshLODModel& NewModel,
															   const FString& SkeletalMeshName ) const
	{
		// Verify that we need to add the alternate weights.
		const bool bSrcHasWeightOverrides = (SrcLODModel.SkinWeightProfiles.Num() != 0);
		if (!bSrcHasWeightOverrides)
		{
			return;
		}

		const auto& SrcSkinWeightProfileData = SrcLODModel.SkinWeightProfiles;

		auto& DstSkinWeightProfileData = NewModel.SkinWeightProfiles;

		// To decode the boneIds stored in the SrcLODModel, we need a map between vertexID and section (and thus BoneMap)

		TArray<int32> SrcVertIdxToSectionMap;
		CreateVertexToSectionMap(SrcLODModel, SrcVertIdxToSectionMap);


		// Add the to the NewModel, the "SourceModelInfluence" arrays for each profile. 

		for (const auto& Profile : SrcSkinWeightProfileData)
		{
			const FName& ProfileName = Profile.Key;
			const FImportedSkinWeightProfileData& SrcImportedProfileData = Profile.Value;
			const TArray<FRawSkinWeight>& SrcBonesAndWeights = SrcImportedProfileData.SkinWeights;
			if (SrcBonesAndWeights.Num() < 1 || SrcImportedProfileData.SourceModelInfluences.Num() < 1)
			{
				//Skip empty profile
				continue;
			}


			// Create the SrcModelInfluences for this profile.
			TArray <SkeletalMeshImportData::FRawBoneInfluence> RawBoneInfluences;

			for (int32 VIdx = 0; VIdx < SkinnedMesh.NumVertices(); ++VIdx)
			{
				// The VerId in the Source Mesh that was closest to the simplified vertex
				const int32 ClosestSrcVertId = SkinnedMesh.VertexBuffer[VIdx].ClosestSrcVertIndex;

				if (ClosestSrcVertId != INDEX_NONE)  // NB: ClosestSrcVertId should never be INDEX_NONE
				{
					const FRawSkinWeight& SrcRawSkinWeight = SrcBonesAndWeights[ClosestSrcVertId];

					// Get the BoneMap that was used to encode these weights. 
					const auto& BoneMap = SrcLODModel.Sections[SrcVertIdxToSectionMap[ClosestSrcVertId]].BoneMap;

					// Add the non-zero weights 
					for (int32 b = 0; b < MAX_TOTAL_INFLUENCES; ++b)
					{
						FBoneIndexType LocalBoneId = SrcRawSkinWeight.InfluenceBones[b];
						uint16 Weight = SrcRawSkinWeight.InfluenceWeights[b];

						if (Weight > 0 && ensure(BoneMap.IsValidIndex(LocalBoneId)))
						{
							// decode the bone weight
							int32 BoneId = BoneMap[LocalBoneId];
							
							SkeletalMeshImportData::FRawBoneInfluence RawBoneInfluence = { (float)Weight / UE::AnimationCore::MaxRawBoneWeightFloat,  VIdx, BoneId };

							RawBoneInfluences.Add(RawBoneInfluence);
						}
					}
				}
			}

			// Pre-process the influences.  This is required for BuildSkeletalMesh to work correctly.
			FLODUtilities::ProcessImportMeshInfluences(SkinnedMesh.NumIndices() /* = SkeletalMeshData.Wedges.Num()*/, RawBoneInfluences, SkeletalMeshName);


			// Make an output array for this profile.
			FImportedSkinWeightProfileData& DstImportedProfileData = DstSkinWeightProfileData.Add(ProfileName, FImportedSkinWeightProfileData());


			// Copy the cleaned up data into the ImportedProfileData.  This is really a translation step since
			// FVertInfluence and FRawBoneInfluence use different storage types for the bone ID.
			TArray<SkeletalMeshImportData::FVertInfluence>& DstSrcModelInfluences = DstImportedProfileData.SourceModelInfluences;
			{
				DstSrcModelInfluences.Empty(RawBoneInfluences.Num());
				DstSrcModelInfluences.AddUninitialized(RawBoneInfluences.Num());

				for (int32 i = 0; i < RawBoneInfluences.Num(); ++i)
				{
					const SkeletalMeshImportData::FRawBoneInfluence& RawBoneInfluence = RawBoneInfluences[i];
					SkeletalMeshImportData::FVertInfluence& DstSrcInfluence           = DstSrcModelInfluences[i];

					DstSrcInfluence.Weight    = RawBoneInfluence.Weight;
					DstSrcInfluence.VertIndex = RawBoneInfluence.VertexIndex;
					DstSrcInfluence.BoneIndex = (FBoneIndexType)RawBoneInfluence.BoneIndex;
				}
			}
		}
	}

	void  FQuadricSkeletalMeshReduction::UpdateAlternateWeights(const int32 MaxBonesPerVertex, FSkeletalMeshLODModel& LODModel) const
	{
		typedef SkeletalSimplifier::VertexTypes::BoneSparseVertexAttrs  BoneIdWeightMap;

		auto& SkinWeightProfileData = LODModel.SkinWeightProfiles;
		
		// Verify that we need to add the alternate weights.
		const bool bSrcHasWeightOverrides = (SkinWeightProfileData.Num() != 0);
		if (!bSrcHasWeightOverrides)
		{
			return;
		}

		// The number of verts in the 'pre-chunked source'
		const int32 NumImportVertex = LODModel.MaxImportVertex + 1;


		// Create a map between the VertexID and the Section
		TArray<int32> VertIdxToSectionMap;
		CreateVertexToSectionMap(LODModel, VertIdxToSectionMap);


		for (auto& Profile : SkinWeightProfileData)
		{
			FImportedSkinWeightProfileData& ImportedProfileData = Profile.Value;
			const TArray<SkeletalMeshImportData::FVertInfluence>& SrcModelInfluences = ImportedProfileData.SourceModelInfluences;

			//  Create a structure that allows us to look-up by SourceModel Vertex ID 
			
			TArray<BoneIdWeightMap> VtxToBoneIdWeightMap;
			VtxToBoneIdWeightMap.AddDefaulted(NumImportVertex);

			for (int32 i = 0; i < SrcModelInfluences.Num(); ++i)
			{
				const SkeletalMeshImportData::FVertInfluence& VertInfluence = SrcModelInfluences[i];
				const int32 VtxId = VertInfluence.VertIndex;
				if (VtxId < NumImportVertex)
				{
					VtxToBoneIdWeightMap[VtxId].SetElement(VertInfluence.BoneIndex, VertInfluence.Weight);
				}
			}

			// sort the bones and limit to MaxBonesPerVertex
			for (int32 i = 0; i < NumImportVertex; ++i)
			{
				VtxToBoneIdWeightMap[i].Correct(MaxBonesPerVertex); 
			}

			// SkinWeights we need to populate.
			TArray<FRawSkinWeight>& SkinWeights = ImportedProfileData.SkinWeights;
			SkinWeights.Empty(LODModel.NumVertices);
			SkinWeights.AddUninitialized(LODModel.NumVertices);

			// map the verts in the LOD model to the input order.
			const TArray<uint32>& ImportVertexMap = LODModel.GetRawPointIndices();
			check(ImportVertexMap.Num() == (int32)LODModel.NumVertices);
			int32 MaxNumInfluences = 0;
			for (int32 VertexIndex = 0; VertexIndex < (int32)LODModel.NumVertices; ++VertexIndex)
			{
				// Map to section and to imported vertex

				const int32 SectionId = VertIdxToSectionMap[VertexIndex];
				const int32 SrcVertId = ImportVertexMap[VertexIndex];

				FSkelMeshSection& Section = LODModel.Sections[SectionId];
				// The BoneMap for this section, needed to encode bones.

				const auto& BoneMap = Section.BoneMap;

				// the dst for the bones and weights.

				FRawSkinWeight& WeightAndBones = SkinWeights[VertexIndex];
				ZeroFRawSkinWeight(WeightAndBones);

				// The bones and Weights for this vertex.
				{
					const BoneIdWeightMap& BoneWeight = VtxToBoneIdWeightMap[SrcVertId];

					// Add each bone / weight. 
					// keep track of the total weight.  should sum to 255 and the first weight is the largest
					int32 InfluenceIndex = 0;
					FBoneIndexType InfluenceBones[MAX_TOTAL_INFLUENCES];
					float InfluenceWeights[MAX_TOTAL_INFLUENCES];
					for (const auto& Pair : BoneWeight.GetData())
					{
						const int32 BoneId = Pair.Key;
						const int32 LocalBoneId = BoneMap.Find(BoneId);
						if (ensure(LocalBoneId != INDEX_NONE))
						{
							InfluenceBones[InfluenceIndex] = LocalBoneId;
							InfluenceWeights[InfluenceIndex] = static_cast<float>(Pair.Value);
							InfluenceIndex++;
						}
					}
					
					using namespace UE::AnimationCore;
					FBoneWeights QuantizedWeights = FBoneWeights::Create(InfluenceBones, InfluenceWeights, InfluenceIndex);
					
					if (QuantizedWeights.Num() > MaxNumInfluences)
					{
						MaxNumInfluences = QuantizedWeights.Num();
						if (MaxNumInfluences > Section.GetMaxBoneInfluences())
						{
							Section.MaxBoneInfluences = MaxNumInfluences;
						}
					}

					InfluenceIndex = 0;
					for (FBoneWeight QuantizedWeight: QuantizedWeights)
					{
						WeightAndBones.InfluenceBones[InfluenceIndex] = QuantizedWeight.GetBoneIndex();
						WeightAndBones.InfluenceWeights[InfluenceIndex] = QuantizedWeight.GetRawWeight();
						InfluenceIndex++;
					}
				}
			}
			
		}

	}



void FQuadricSkeletalMeshReduction::ConvertToFSkeletalMeshLODModel(
	const FString& SkeletalMeshName,
	const int32 MaxBonesPerVertex,
	const FSkeletalMeshLODModel& SrcLODModel,
	const SkeletalSimplifier::FSkinnedSkeletalMesh& SkinnedMesh,
	const FReferenceSkeleton& RefSkeleton,
	const TArray<FVertexAttributeInfo>& VertexAttributeInfos,
	FSkeletalMeshLODModel& NewModel,
	const bool bReducingSourceModel,
	const ITargetPlatform* TargetPlatform) const
{
	// We might be re-using this model - so clear it.

	Empty(NewModel);

	// Convert the mesh to a struct of arrays
	
	FSkeletalMeshData SkeletalMeshData;
	ExtractFSkeletalData(SkinnedMesh, SkeletalMeshData);

	//if (!bReducingSourceModel)
	{
		// Add alternate weight data to the NewModel.  Note, this has to be done before we "BuildSkeletalMesh" to insure
		// that the bone-based vertex chunking respects the alternate weights.
		// NB: this only prepares the NewModel, but BuildSkeletalMesh is only 1/2-aware of this, so we will have to do some additional work after.

		AddSourceModelInfluences(SrcLODModel, SkinnedMesh, NewModel, SkeletalMeshName);
	}

	// Create dummy map of 'point to original'

	TArray<int32> DummyMap;
	DummyMap.AddUninitialized(SkeletalMeshData.Points.Num());
	for (int32 PointIdx = 0; PointIdx < SkeletalMeshData.Points.Num(); PointIdx++)
	{
		DummyMap[PointIdx] = PointIdx;
	}

	// Make sure we do not recalculate normals or remove any degenerated data (threshold force to zero)

	IMeshUtilities::MeshBuildOptions Options;
	Options.bComputeNormals = false;
	Options.bComputeTangents = false;
	Options.bUseMikkTSpace = true; //Avoid builtin build by specifying true for mikkt space
	Options.bComputeWeightedNormals = false;
	//Leave the default threshold
	Options.bRemoveDegenerateTriangles = false;
	Options.TargetPlatform = TargetPlatform;
	IMeshUtilities& MeshUtilities = FModuleManager::Get().LoadModuleChecked<IMeshUtilities>("MeshUtilities");
	
	// Create skinning streams for NewModel.

	MeshUtilities.BuildSkeletalMesh(
		NewModel,
		SkeletalMeshName,
		RefSkeleton,
		SkeletalMeshData.Influences,
		SkeletalMeshData.Wedges,
		SkeletalMeshData.Faces,
		SkeletalMeshData.Points,
		DummyMap,
		Options
	);

	//Re-Apply the user section changes, the UserSectionsData is map to original section and should match the builded LODModel
	NewModel.SyncronizeUserSectionsDataArray();

	// Set texture coordinate count on the new model.
	NewModel.NumTexCoords = SkeletalMeshData.TexCoordCount;

	// Copy back the vertex attributes, if any. Use the MeshToImportVertexMap on the NewModel to map from the new point location to the
	// one in the reduction mesh.
	for (const FVertexAttributeInfo& AttributeInfo: VertexAttributeInfos)
	{
		FSkeletalMeshModelVertexAttribute& VertexAttribute = NewModel.VertexAttributes.FindOrAdd(AttributeInfo.AttributeName);

		VertexAttribute.ComponentCount = AttributeInfo.ComponentCount;
		VertexAttribute.Values.Reserve(SkinnedMesh.NumVertices() * VertexAttribute.ComponentCount);

		for (int32 Index = 0; Index < SkinnedMesh.NumVertices(); Index++)
		{
			const int32 OriginalIndex = NewModel.GetRawPointIndices()[Index];
			const auto& Vertex = SkinnedMesh.VertexBuffer[OriginalIndex];
			for (int32 Component = 0; Component < VertexAttribute.ComponentCount; Component++)
			{
				VertexAttribute.Values.Add(static_cast<float>(Vertex.AdditionalAttributes.GetElement(AttributeInfo.Offset + Component)));
			}
		}
	}

	

	//if (!bReducingSourceModel)
	{
		// Update the alternate weights

		UpdateAlternateWeights(MaxBonesPerVertex, NewModel);
	}
}

bool FQuadricSkeletalMeshReduction::ReduceSkeletalLODModel( const FSkeletalMeshLODModel& SrcModel,
														    FSkeletalMeshLODModel& OutSkeletalMeshLODModel,
														    const FString& SkeletalMeshName,
	                                                        const FBoxSphereBounds& Bounds,
	                                                        const FReferenceSkeleton& RefSkeleton,
	                                                        FSkeletalMeshOptimizationSettings Settings,
															const FPrioritizedFeatures& PrioritizedFeatures,
	                                                        const TArray<FMatrix>& BoneMatrices,
	                                                        const int32 LODIndex,
															const bool bReducingSourceModel,
														    const ITargetPlatform* TargetPlatform
                                                           ) const
{

	const uint32 SrcNumVerts = SrcModel.NumVertices;
	auto GetLODModelTriangleCount = [](const FSkeletalMeshLODModel& LODModel)->uint32
	{
		uint32 NumTriangles = 0;
		for (const FSkelMeshSection& Section : LODModel.Sections)
		{
			NumTriangles += Section.NumTriangles;
		}
		return NumTriangles;
	};
	const uint32 SrcNumTriangles = GetLODModelTriangleCount(SrcModel);

	const int32 MaxBonesPerVert = FMath::Clamp(Settings.MaxBonesPerVertex, 0, MAX_TOTAL_INFLUENCES);
	
	// Parameters for Simplification etc
	const bool bUseVertexPercentCriterion   = ((Settings.TerminationCriterion == SMTC_NumOfVerts     || Settings.TerminationCriterion == SMTC_TriangleOrVert) && (Settings.NumOfVertPercentage < 1.f || Settings.MaxNumOfVertsPercentage < SrcNumVerts)) ;
	const bool bUseTrianglePercentCriterion = ((Settings.TerminationCriterion == SMTC_NumOfTriangles || Settings.TerminationCriterion == SMTC_TriangleOrVert) && (Settings.NumOfTrianglesPercentage < 1.f || Settings.MaxNumOfTrianglesPercentage < SrcNumTriangles));

	const bool bUseMaxVertexCriterion   = ((Settings.TerminationCriterion == SMTC_AbsNumOfVerts || Settings.TerminationCriterion == SMTC_AbsTriangleOrVert) && SrcNumVerts);
	const bool bUseMaxTriangleCriterion = ((Settings.TerminationCriterion == SMTC_AbsNumOfTriangles || Settings.TerminationCriterion == SMTC_AbsTriangleOrVert) && Settings.MaxNumOfTriangles < INT32_MAX);

	const bool bProcessGeometry      = (bUseTrianglePercentCriterion || bUseVertexPercentCriterion || bUseMaxTriangleCriterion || bUseMaxVertexCriterion);
	const bool bProcessBones         = (MaxBonesPerVert < MAX_TOTAL_INFLUENCES);
	
	bool bOptimizeMesh         = (bProcessGeometry || bProcessBones);

	if (bOptimizeMesh)
	{
		UE_LOGF(LogSkeletalMeshReduction, Log, "Reducing skeletal mesh for LOD %d ", LODIndex);
	}
	
	// Generate a single skinned mesh form the SrcModel.  This mesh has per-vertex tangent space.
	TArray<int32> VertToOriginalSectionId;
	TArray<FVertexAttributeInfo> VertexAttributeInfos;
	SkeletalSimplifier::FSkinnedSkeletalMesh SkinnedSkeletalMesh;
	ConvertToFSkinnedSkeletalMesh(SrcModel, BoneMatrices, LODIndex, SkinnedSkeletalMesh, SkeletalMeshName, VertexAttributeInfos, &VertToOriginalSectionId);

	int32 IterationNum = 0;
	//We keep the original MaxNumVerts because if we iterate we want to still compare with the original request.
	uint32 OriginalMaxNumVertsSetting = bUseVertexPercentCriterion ? Settings.MaxNumOfVertsPercentage : Settings.MaxNumOfVerts;
	do 
	{
		if (bOptimizeMesh)
		{

			// Add specialized weights for verts associated with "important" bones or sections
			UpdateSpecializedVertWeights(PrioritizedFeatures, VertToOriginalSectionId, SkinnedSkeletalMesh);

			if (bProcessGeometry)
			{
				// Capture the UV bounds from the source mesh.
				FVector2f  UVBounds[2 * SkeletalSimplifier::MeshVertType::BasicAttrContainerType::NumUVs];
				ComputeUVBounds(SkinnedSkeletalMesh, UVBounds);
			
				// Use the bone-aware simplifier
				SimplifyMesh(Settings, Bounds, SkinnedSkeletalMesh);

				// Clamp the UVs of the simplified mesh to match the source mesh.
				ClampUVBounds(UVBounds, SkinnedSkeletalMesh);
			}

			// Reduce the number of bones per-vert
			if (bProcessBones)
			{
				TrimBonesPerVert(SkinnedSkeletalMesh, MaxBonesPerVert);
			}
		}

		// Convert to SkeletalMeshLODModel. 
		ConvertToFSkeletalMeshLODModel(
			SkeletalMeshName, Settings.MaxBonesPerVertex, SrcModel, SkinnedSkeletalMesh, RefSkeleton, 
			VertexAttributeInfos, OutSkeletalMeshLODModel, bReducingSourceModel, TargetPlatform);

		// We may need to do additional simplification if the user specified a hard number limit for verts and
		// the internal chunking during conversion split some verts.
		if ((bUseMaxVertexCriterion || bUseVertexPercentCriterion) && OutSkeletalMeshLODModel.NumVertices > OriginalMaxNumVertsSetting && OutSkeletalMeshLODModel.NumVertices > 6)
		{
			const bool bTerminatedOnVertCount = ((Settings.TerminationCriterion == SMTC_NumOfVerts || Settings.TerminationCriterion == SMTC_TriangleOrVert) && (OutSkeletalMeshLODModel.NumVertices > OriginalMaxNumVertsSetting)) ||
												(Settings.TerminationCriterion == SMTC_AbsNumOfVerts) ||
				                                (Settings.TerminationCriterion == SMTC_AbsTriangleOrVert && !((uint32)SkinnedSkeletalMesh.NumIndices() / (uint32)3 <= Settings.MaxNumOfTriangles));
			  
			if (bTerminatedOnVertCount)
			{
				// Some verts were created by chunking - we need simplify more.
				uint32 ExcessVerts = OutSkeletalMeshLODModel.NumVertices - OriginalMaxNumVertsSetting;
				if (bUseVertexPercentCriterion)
				{
					Settings.MaxNumOfVertsPercentage = FMath::Max(Settings.MaxNumOfVertsPercentage - ExcessVerts, (uint32)6);
				}
				else
				{
					Settings.MaxNumOfVerts = FMath::Max(Settings.MaxNumOfVerts - ExcessVerts, (uint32)6);
				}

				UE_LOGF(LogSkeletalMeshReduction, Log, "Chunking to limit unique bones per section generated additional vertices - continuing simplification of LOD %d ", LODIndex);
				ConvertToFSkinnedSkeletalMesh(SrcModel, BoneMatrices, LODIndex, SkinnedSkeletalMesh, SkeletalMeshName, VertexAttributeInfos);
			}
			else
			{
				bOptimizeMesh = false;
			}

			IterationNum++;
		}
		else
		{
			bOptimizeMesh = false;
		}
	} 
	while (bOptimizeMesh && IterationNum < 5);

	bool bReturnValue =  (OutSkeletalMeshLODModel.NumVertices > 0);
	

	return bReturnValue;
}


void FQuadricSkeletalMeshReduction::ReduceSkeletalMesh(USkeletalMesh& SkeletalMesh, FSkeletalMeshModel& SkeletalMeshResource, int32 LODIndex, const ITargetPlatform* TargetPlatform) const
{
	check(LODIndex <= SkeletalMeshResource.LODModels.Num());

	//If the Current LOD is an import from file
	bool bOldLodWasFromFile = SkeletalMesh.IsValidLODIndex(LODIndex) && SkeletalMesh.GetLODInfo(LODIndex)->bHasBeenSimplified == false;

	//True if the LOD is added by this reduction
	bool bLODModelAdded = false;

	// Insert a new LOD model entry if needed.
	if (LODIndex == SkeletalMeshResource.LODModels.Num())
	{
		FSkeletalMeshLODModel* ModelPtr = nullptr;
		SkeletalMeshResource.LODModels.Add(ModelPtr);
		bLODModelAdded = true;
	}

	// Copy over LOD info from LOD0 if there is no previous info.
	if (LODIndex == SkeletalMesh.GetLODNum())
	{
		// if there is no LOD, add one more
		SkeletalMesh.AddLODInfo();
	}


	// Swap in a new model, delete the old.

	FSkeletalMeshLODModel** LODModels = SkeletalMeshResource.LODModels.GetData();


	// get settings
	FSkeletalMeshLODInfo* LODInfo = SkeletalMesh.GetLODInfo(LODIndex);
	const FSkeletalMeshOptimizationSettings& Settings = LODInfo->ReductionSettings;
	

	FSkeletalMeshLODModel DstModelBackup;
	if (!bLODModelAdded)
	{
		DstModelBackup.Sections = SkeletalMeshResource.LODModels[LODIndex].Sections;
		DstModelBackup.UserSectionsData = SkeletalMeshResource.LODModels[LODIndex].UserSectionsData;
	}

	// Struct to identify important bones.  Vertices associated with these bones
	// will have additional collapse weight added to them.

	FPrioritizedFeatures  PrioritizedFeatures;
	{
		
		const float PrioritizationWeight = LODInfo->WeightOfPrioritization;
		PrioritizedFeatures.Weight = PrioritizationWeight;

		const TArray<FBoneReference>& BonesToPrioritize = LODInfo->BonesToPrioritize;
		for (const FBoneReference& BoneReference : BonesToPrioritize)
		{
			int32 BoneId = SkeletalMesh.GetRefSkeleton().FindRawBoneIndex(BoneReference.BoneName);

			// Q: should we exclude BoneId = 0?
			PrioritizedFeatures.BoneIds.AddUnique(BoneId);
		}

		const TArray<FSectionReference>& SectionsToPrioritize = LODInfo->SectionsToPrioritize;
		for (const FSectionReference& SectionReference : SectionsToPrioritize)
		{
			if (SectionReference.IsValidToEvaluate(SkeletalMeshResource.LODModels[LODIndex]))
			{
				PrioritizedFeatures.SectionIds.AddUnique(SectionReference.SectionIndex);
			}
		}
	}
	// select which mesh we're reducing from
	// use BaseLOD
	int32 BaseLOD = 0;
	FSkeletalMeshModel* SkelResource = SkeletalMesh.GetImportedModel();
	FSkeletalMeshLODModel* SrcModel = &SkelResource->LODModels[0];

	// only allow to set BaseLOD if the LOD is less than this
	if (Settings.BaseLOD > 0)
	{
		
		if (Settings.BaseLOD == LODIndex && !SkeletalMesh.HasMeshDescription(LODIndex))
		{
			//Cannot reduce ourself if we are not imported
			UE_ASSET_LOG(LogSkeletalMeshReduction, Warning, &SkeletalMesh, TEXT("Building LOD %d - Cannot generate LOD with itself if the LOD do not have imported Data. Using Base LOD 0 instead"), LODIndex);
		}
		else if (Settings.BaseLOD <= LODIndex && SkeletalMeshResource.LODModels.IsValidIndex(Settings.BaseLOD))
		{
			BaseLOD = Settings.BaseLOD;
			SrcModel = &SkeletalMeshResource.LODModels[BaseLOD];
		}
		else
		{
			// warn users
			UE_ASSET_LOG(LogSkeletalMeshReduction, Warning, &SkeletalMesh, TEXT("Building LOD %d - Invalid Base LOD entered. Using Base LOD 0 instead"), LODIndex);
		}
	}

	//We backup only the sections and the user sections data
	FSkeletalMeshLODModel SrcModelBackup;
	SrcModelBackup.Sections = SrcModel->Sections;
	SrcModelBackup.UserSectionsData = SrcModel->UserSectionsData;

	//Restore the source sections data
	auto RestoreUserSectionsData = [](const FSkeletalMeshLODModel& SourceLODModel, FSkeletalMeshLODModel& DestinationLODModel, bool bAddMissingUserSectionData)
	{
		//Now restore the reduce section user change and adjust the originalDataSectionIndex to point on the correct UserSectionData
		TArray<int32> SourceSectionMatched;
		SourceSectionMatched.Reserve(SourceLODModel.Sections.Num());
		for (int32 SourceSectionIndex = 0; SourceSectionIndex < SourceLODModel.Sections.Num(); ++SourceSectionIndex)
		{
			SourceSectionMatched.Add(INDEX_NONE);
		}

		for (int32 SectionIndex = 0; SectionIndex < DestinationLODModel.Sections.Num(); ++SectionIndex)
		{
			FSkelMeshSection& Section = DestinationLODModel.Sections[SectionIndex];
			FSkelMeshSourceSectionUserData& DestinationUserData = FSkelMeshSourceSectionUserData::GetSourceSectionUserData(DestinationLODModel.UserSectionsData, Section);
			for (int32 SourceSectionIndex = 0; SourceSectionIndex < SourceLODModel.Sections.Num(); ++SourceSectionIndex)
			{
				if (SourceSectionMatched[SourceSectionIndex] != INDEX_NONE)
				{
					continue;
				}
				const FSkelMeshSection& SourceSection = SourceLODModel.Sections[SourceSectionIndex];
				if (const FSkelMeshSourceSectionUserData* SourceUserData = SourceLODModel.UserSectionsData.Find(SourceSection.OriginalDataSectionIndex))
				{
					//Section material index reflect the imported material slot, if the user change this value it will go in 
					//FSkeletalMeshLODInfo::LODMaterialMap of the skeletalmesh. So we can count on the material index to be unique per
					//section that are not chunked.
					if (Section.MaterialIndex == SourceSection.MaterialIndex)
					{
						DestinationUserData = *SourceUserData;
						SourceSectionMatched[SourceSectionIndex] = SectionIndex;
						break;
					}
				}
			}
		}
		//If the reduction result have less "user section data" compare to the source, we have to make correction to the data
		//and copy back the source "user section data" into the destination
		const bool bHasSomeMissingData = DestinationLODModel.UserSectionsData.Num() < SourceLODModel.UserSectionsData.Num();
		if (bAddMissingUserSectionData && bHasSomeMissingData)
		{
			//The goal is to reamp all destination LOD model sections FSkelMeshSection::OriginalDataSectionIndex to fit with the
			//original SourceModel FSkeletalMeshLODModel::UserSectionData. to do this we will use an offset we increment for any unmatched
			//source section.
			int32 OriginalSectionIndexOffset = 0;
			for (int32 SourceSectionIndex = 0; SourceSectionIndex < SourceLODModel.Sections.Num(); ++SourceSectionIndex)
			{
				const FSkelMeshSection& SourceSection = SourceLODModel.Sections[SourceSectionIndex];
				//Skip chunked section, they do not affect the user section data
				if (SourceSection.ChunkedParentSectionIndex != INDEX_NONE)
				{
					continue;
				}
				//Skip the sections that already have the correct value, no user data section was missing yet
				if (OriginalSectionIndexOffset == 0 && SourceSectionMatched[SourceSectionIndex] == SourceSectionIndex)
				{
					//Nothing to change
					continue;
				}
				if (SourceSectionMatched[SourceSectionIndex] == INDEX_NONE)
				{
					//We need to increment the offset for every unmatched source section
					OriginalSectionIndexOffset++;
				}
				else
				{
					//Fix up the current section by adding the offset to the section OriginalDataSectionIndex.
					FSkelMeshSection& Section = DestinationLODModel.Sections[SourceSectionMatched[SourceSectionIndex]];
					Section.OriginalDataSectionIndex += OriginalSectionIndexOffset;
				}
			}
			//We can now copy the user section data of the source since all destination sections has been converted to use the source UserSectionsData
			DestinationLODModel.UserSectionsData = SourceLODModel.UserSectionsData;
		}

		DestinationLODModel.SyncronizeUserSectionsDataArray();
	};

	FString BackupLodModelBuildStringID = TEXT("");
	FString BackupRawSkeletalMeshBulkDataID = TEXT("");

	// Unbind any existing clothing assets before we reimport the geometry
	TArray<ClothingAssetUtils::FClothingAssetMeshBinding> ClothingBindings;
	//Do not play with cloth if the LOD is added
	if (!bLODModelAdded)
	{
		FLODUtilities::UnbindClothingAndBackup(&SkeletalMesh, ClothingBindings, LODIndex);
		//We have to put back the exact UserSectionsData to not invalidate the DDC key
		BackupLodModelBuildStringID = SkeletalMeshResource.LODModels[LODIndex].BuildStringID;
		BackupRawSkeletalMeshBulkDataID = SkeletalMeshResource.LODModels[LODIndex].RawSkeletalMeshBulkDataID;
	}

	bool bReducingSourceModel = false;
	//Reducing source LOD, we need to use the LOD import data so it can be iterative
	if (BaseLOD == LODIndex && SkeletalMesh.HasMeshDescription(LODIndex))
	{
		bReducingSourceModel = true;
	}
	else
	{
		check(BaseLOD < LODIndex);
	}

	check(SrcModel);

	// now try bone reduction process if it's setup
	TMap<FBoneIndexType, FBoneIndexType> BonesToRemove;
	
	IMeshBoneReduction* MeshBoneReductionInterface = FModuleManager::Get().LoadModuleChecked<IMeshBoneReductionModule>("MeshBoneReduction").GetMeshBoneReductionInterface();

	TArray<FName> BoneNames;
	const int32 NumBones = SkeletalMesh.GetRefSkeleton().GetNum();
	for (int32 BoneIndex = 0; BoneIndex < NumBones; ++BoneIndex)
	{
		BoneNames.Add(SkeletalMesh.GetRefSkeleton().GetBoneName(BoneIndex));
	}

	// get the relative to ref pose matrices
	TArray<FMatrix> RelativeToRefPoseMatrices;
	RelativeToRefPoseMatrices.AddDefaulted(NumBones);

	// Set initial matrices to identity
	for (int32 Index = 0; Index < NumBones; ++Index)
	{
		RelativeToRefPoseMatrices[Index] = FMatrix::Identity;
	}

	// if it has bake pose, gets ref to local matrices using bake pose
	if (const UAnimSequence* BakePoseAnim = SkeletalMesh.GetBakePose(LODIndex))
	{
		FMemMark Mark(FMemStack::Get());
		
		const FReferenceSkeleton& RefSkeleton = SkeletalMesh.GetRefSkeleton();

		// Get component space retarget base pose, will be equivalent of ref-pose if not edited
		TArray<FTransform> ComponentSpaceRefPose;
		FAnimationRuntime::FillUpComponentSpaceTransforms(SkeletalMesh.GetRefSkeleton(), SkeletalMesh.GetRefSkeleton().GetRefBonePose(), ComponentSpaceRefPose);
		
		// Retrieve bone indices which will be removed
		if (MeshBoneReductionInterface != nullptr)
		{
			MeshBoneReductionInterface->GetBoneReductionData(&SkeletalMesh, LODIndex, BonesToRemove);
		}

		// Setup BoneContainer and CompactPose
		TArray<FBoneIndexType> RequiredBoneIndexArray;
		RequiredBoneIndexArray.AddUninitialized(RefSkeleton.GetNum());
		for (int32 BoneIndex = 0; BoneIndex < RequiredBoneIndexArray.Num(); ++BoneIndex)
		{
			RequiredBoneIndexArray[BoneIndex] = BoneIndex;
		}

		FBoneContainer RequiredBones(RequiredBoneIndexArray, UE::Anim::ECurveFilterMode::DisallowAll, SkeletalMesh);
		RequiredBones.SetUseRAWData(true);

		FCompactPose Pose;
		Pose.SetBoneContainer(&RequiredBones);
		Pose.ResetToRefPose();

		FBlendedCurve TempCurve;
		TempCurve.InitFrom(RequiredBones);
		UE::Anim::FStackAttributeContainer TempAttributes;
		
		FAnimationPoseData AnimPose(Pose, TempCurve, TempAttributes);

		// Retrieve animated pose from anim sequence (including retargeting)
		const USkeleton* Skeleton = SkeletalMesh.GetSkeleton();
		const FName RetargetSource = Skeleton->GetRetargetSourceForMesh(&SkeletalMesh);
		UE::Anim::BuildPoseFromModel(BakePoseAnim->GetDataModel(), AnimPose, 0.0, EAnimInterpolationType::Step, RetargetSource, Skeleton->GetRefLocalPoses(RetargetSource));
		
		// Calculate component space animated pose matrices
		TArray<FMatrix> ComponentSpaceAnimatedPose;
		ComponentSpaceAnimatedPose.AddDefaulted(NumBones);

		for (int32 BoneIndex = 0; BoneIndex < NumBones; ++BoneIndex)
		{
			const FCompactPoseBoneIndex PoseBoneIndex(BoneIndex);
			const int32 ParentIndex = RefSkeleton.GetParentIndex(BoneIndex);
			if (ParentIndex != INDEX_NONE)
			{
				// If the bone will be removed, get the local-space retarget-ed animation bone transform
				if (BonesToRemove.Contains(BoneIndex))
				{
					ComponentSpaceAnimatedPose[BoneIndex] = Pose[PoseBoneIndex].ToMatrixWithScale() * ComponentSpaceAnimatedPose[ParentIndex];
				}
				// Otherwise use the component-space retarget base pose transform
				else
				{
					ComponentSpaceAnimatedPose[BoneIndex] = ComponentSpaceRefPose[BoneIndex].ToMatrixWithScale();
				}
			}
			else
			{
				// If the bone will be removed, get the retarget-ed animation bone transform
				if (BonesToRemove.Contains(BoneIndex))
				{
					ComponentSpaceAnimatedPose[BoneIndex] = Pose[PoseBoneIndex].ToMatrixWithScale();
				}
				// Otherwise use the retarget base pose transform
				else
				{
					ComponentSpaceAnimatedPose[BoneIndex] = ComponentSpaceRefPose[BoneIndex].ToMatrixWithScale();
				}
			}
		}

		// Calculate relative to retarget base (ref) pose matrix
		for (int32 BoneIndex = 0; BoneIndex < NumBones; ++BoneIndex)
		{
			RelativeToRefPoseMatrices[BoneIndex] = ComponentSpaceRefPose[BoneIndex].ToMatrixWithScale().Inverse() * ComponentSpaceAnimatedPose[BoneIndex];
		}

		// Reset map for later usage
		BonesToRemove.Empty();
	}

	FSkeletalMeshLODModel* NewModel = new FSkeletalMeshLODModel();

	// Swap out the old model.  
	bool bPutBackRawMesh = false;
	{
		FSkeletalMeshLODModel* Old = LODModels[LODIndex];

		LODModels[LODIndex] = NewModel;

		if (!bReducingSourceModel && Old)
		{
			bool bIsOldRawSkelMeshEmpty = Old->bIsRawSkeletalMeshBulkDataEmpty;
			//We need to backup the original RawSkeletalMeshBulkData in case it was an imported LOD
			if (!bLODModelAdded && !bIsOldRawSkelMeshEmpty)
			{
				bPutBackRawMesh = true;
			}

			if (Settings.OnDeleteLODModelDelegate.IsBound())
			{
				Settings.OnDeleteLODModelDelegate.Execute(Old);
			}
			else
			{
				// Make sure the deletion is happening on the game-thread.
				// Deleting a structure containing bulkdata can crash when the bulkdata is detached from the archive.
				if (IsInGameThread())
			{
				delete Old;
			}
			else
			{
					Async(EAsyncExecution::TaskGraphMainThread, [Old]() { delete Old; });
				}
			}
		}
		else if(bReducingSourceModel)
		{
			bPutBackRawMesh = true;
		}
	}

	// Reduce LOD model with SrcMesh if src mesh has more then 1 triangle
	if (SrcModel->NumVertices > 3 && ReduceSkeletalLODModel(*SrcModel, *NewModel, SkeletalMesh.GetPathName(), SkeletalMesh.GetImportedBounds(), SkeletalMesh.GetRefSkeleton(), Settings, PrioritizedFeatures, RelativeToRefPoseMatrices, LODIndex, bReducingSourceModel, TargetPlatform))
	{
		FSkeletalMeshLODInfo* ReducedLODInfoPtr = SkeletalMesh.GetLODInfo(LODIndex);
		check(ReducedLODInfoPtr);
		// Do any joint-welding / bone removal.

		if (MeshBoneReductionInterface != nullptr && MeshBoneReductionInterface->GetBoneReductionData(&SkeletalMesh, LODIndex, BonesToRemove))
		{
			// fix up chunks to remove the bones that set to be removed
			for (int32 SectionIndex = 0; SectionIndex < NewModel->Sections.Num(); ++SectionIndex)
			{
				MeshBoneReductionInterface->FixUpSectionBoneMaps(NewModel->Sections[SectionIndex], BonesToRemove, NewModel->SkinWeightProfiles);
			}
		}

		if (bOldLodWasFromFile)
		{
			ReducedLODInfoPtr->LODMaterialMap.Empty();
		}
		// Flag this LOD as having been simplified.
		ReducedLODInfoPtr->bHasBeenSimplified = true;

		//Restore the user sections data to what it was. It must be done if we want to avoid changing the DDC key. I.E. UserSectionData is part of the key
		//DDC key cannot be change during the build
		{
			FSkeletalMeshLODModel& ImportedModelLOD = SkeletalMesh.GetImportedModel()->LODModels[LODIndex];

			if (!bLODModelAdded)
			{
				//We have to force the UserSectionData to be the one from the backup
				constexpr bool bAddMissingUserSectionData = true;
				RestoreUserSectionsData(DstModelBackup, ImportedModelLOD, bAddMissingUserSectionData);
				//If its an existing LOD put back the buildStringID
				ImportedModelLOD.BuildStringID = BackupLodModelBuildStringID;
				ImportedModelLOD.RawSkeletalMeshBulkDataID = BackupRawSkeletalMeshBulkDataID;
			}
			else
			{
				constexpr bool bAddMissingUserSectionData = false;
				RestoreUserSectionsData(SrcModelBackup, ImportedModelLOD, bAddMissingUserSectionData);
			}
		}
	}
	else
	{
		FSkeletalMeshLODModel::CopyStructure(NewModel, SrcModel);

		// Do any joint-welding / bone removal.
		if (MeshBoneReductionInterface != nullptr && MeshBoneReductionInterface->GetBoneReductionData(&SkeletalMesh, LODIndex, BonesToRemove))
		{
			// fix up chunks to remove the bones that set to be removed
			for (int32 SectionIndex = 0; SectionIndex < NewModel->Sections.Num(); ++SectionIndex)
			{
				MeshBoneReductionInterface->FixUpSectionBoneMaps(NewModel->Sections[SectionIndex], BonesToRemove, NewModel->SkinWeightProfiles);
			}
		}

		//Clean up some section data

		for (int32 SectionIndex = SrcModel->Sections.Num() - 1; SectionIndex >= 0; --SectionIndex)
		{
			//New model should be reset to -1 value
			NewModel->Sections[SectionIndex].GenerateUpToLodIndex = -1;
			int8 GenerateUpToLodIndex = SrcModel->Sections[SectionIndex].GenerateUpToLodIndex;
			if (GenerateUpToLodIndex != -1 && GenerateUpToLodIndex < LODIndex)
			{
				//Remove the section
				RemoveMeshSection(*NewModel, SectionIndex);
			}
		}

		SkeletalMesh.GetLODInfo(LODIndex)->LODMaterialMap = SkeletalMesh.GetLODInfo(BaseLOD)->LODMaterialMap;

		// Required bones are recalculated later on.

		NewModel->RequiredBones.Empty();
		SkeletalMesh.GetLODInfo(LODIndex)->bHasBeenSimplified = true;
	}
	
	if (!bLODModelAdded)
	{
		//Put back the clothing for this newly reduce LOD
		if (ClothingBindings.Num() > 0)
		{
			FLODUtilities::RestoreClothingFromBackup(&SkeletalMesh, ClothingBindings, LODIndex);
		}
	}
	
	//Put back the original import data, we need it to allow inline reduction and skeletal mesh split workflow
	//It also warranty that we do not change the ddc key
	if (bPutBackRawMesh && !bLODModelAdded)
	{
		//The SaveLODImportdData can change the cache RawSkeletalMeshBulkDataID, so we have to put back the backup after the save
		FSkeletalMeshLODModel& ImportedModelLOD = SkeletalMesh.GetImportedModel()->LODModels[LODIndex];
		ImportedModelLOD.BuildStringID = BackupLodModelBuildStringID;
		ImportedModelLOD.RawSkeletalMeshBulkDataID = BackupRawSkeletalMeshBulkDataID;
	}

	SkeletalMesh.CalculateRequiredBones(SkeletalMeshResource.LODModels[LODIndex], SkeletalMesh.GetRefSkeleton(), &BonesToRemove);
}

#undef LOCTEXT_NAMESPACE
