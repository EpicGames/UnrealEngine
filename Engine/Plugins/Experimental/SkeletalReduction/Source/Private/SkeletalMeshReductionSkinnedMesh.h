// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "SkeletalSimplifierMeshManager.h"


namespace SkeletalSimplifier
{
	/**
	* Simple mesh class templated on vertex type
	* that holds an index and vertex buffer.
	* 
	* Has the ability to compact and remove unused vertices
	*  NB: here 'unused' means vertices that aren't referenced by the index buffer
	*/
	template <typename SimpVertexType>
	class TSkinnedSkeletalMesh
	{
	public:

		typedef SimpVertexType   VertexType;

		TSkinnedSkeletalMesh() :
			IndexBuffer(NULL),
			VertexBuffer(NULL),
			NumTris(0),
			NumVerts(0),
			TexCoordNum(0)
		{}

		~TSkinnedSkeletalMesh()
		{
			Empty();
		}

		/**
		* Constructor allocates index and vertex buffer.
		*/
		TSkinnedSkeletalMesh(int32 NumTriangles, int32 NumVertices) :
			IndexBuffer(NULL),
			VertexBuffer(NULL),
			NumTris(0),
			NumVerts(0),
			TexCoordNum(0)
		{
			Resize(NumTriangles, NumVertices);
		}

		/**
		* Resizes the mesh to new size.
		* NB:  Deletes data already held in mesh.
		*/
		void Resize(int32 NumTriangles, int32 NumVertices)
		{
			delete[] IndexBuffer;
			delete[] VertexBuffer;

			NumTris  = NumTriangles;
			NumVerts = NumVertices;

			IndexBuffer  = new uint32[NumTriangles * 3];
			VertexBuffer = new SimpVertexType[NumVerts];
		}

		/**
		*  Resizes the mesh to size zero.
		*/
		void Empty()
		{
			delete[] IndexBuffer;
			delete[] VertexBuffer;

			IndexBuffer =  NULL;
			VertexBuffer = NULL;

			NumTris = 0;
			NumVerts = 0;
		}

		/**
		* Size of index buffer
		*/
		int32 NumIndices() const { return NumTris * 3; }

		/**
		* Size of vertex buffer
		*/
		int32 NumVertices() const { return NumVerts; }

		/**
		* Number of texture coords on each vertex.
		*/
		int32 TexCoordCount() const { return TexCoordNum; }

		void SetTexCoordCount(int32 c)
		{
			TexCoordNum = c;
		}


		/**
		* Remove vertices that aren't referenced by the index buffer.
		* Also rebuilds the index buffer to account for the removals.
		* Optionally, VertexRemap will map the compact vert index to the original vert index
		*/
		void Compact(TArray<int32>* VertexRemap = nullptr)
		{
			if (IndexBuffer == NULL)
			{
				return;
			}

			int32* Mask = new int32[NumVerts];
			for (int32 i = 0; i < NumVerts; ++i)
			{
				Mask[i] = 0;
			}

			// mark the verts that are being used
			for (int32 i = 0; i < NumTris * 3; ++i)
			{
				uint32 VertId = IndexBuffer[i];
				Mask[VertId] = 1;
			}

			// count the used verts.
			int32 RequiredVertCount = 0;
			for (int32 i = 0; i < NumVerts; ++i)
			{
				RequiredVertCount += Mask[i];
			}


			if (VertexRemap)
			{
				VertexRemap->Empty();
				VertexRemap->SetNumUninitialized(RequiredVertCount);
			}

			// If all the verts are being used, there is nothing to do
			if (RequiredVertCount == NumVerts)
			{
				if (VertexRemap)
				{
					for (int32 i = 0; i < RequiredVertCount; ++i)
					{
						(*VertexRemap)[i] = i;
					}
				}
				delete[] Mask;
				return;
			}

			// stash the pointers to the current buffers
			uint32* OldIndexBuffer          = IndexBuffer;
			SimpVertexType* OldVertexBuffer = VertexBuffer;

			if (OldVertexBuffer != NULL && OldIndexBuffer != NULL)
			{
				int32 OldNumTris  = NumTris;
				int32 OldNumVerts = NumVerts;

				// null the pointers to keep the resize from deleting the arrays.
				IndexBuffer  = NULL;
				VertexBuffer = NULL;

				// Allocate memory for the compacted mesh.
				Resize(NumTris, RequiredVertCount);

				// Copy the verts into the new vertex array

				for (int32 i = 0, j = 0; i < OldNumVerts; ++i)
				{
					if (Mask[i] == 0) continue;

					checkSlow(j < RequiredVertCount);

					VertexBuffer[j] = OldVertexBuffer[i];
					if (VertexRemap)
					{
						(*VertexRemap)[j] = i;
					}
					j++;
				}

				// record offsets the Mask
				// so that Mask[i] will be the number of voids prior to and including i.
				{
					int32 VoidCount = 0;
					for (int32 i = 0; i < OldNumVerts; ++i)
					{
						VoidCount += (1 - Mask[i]);
						Mask[i] = VoidCount;
					}
				}
				check(OldNumTris == NumTris);

				// translate the offsets in the index buffer 
				for (int32 i = 0; i < OldNumTris * 3; ++i)
				{
					int32 OldVertIdx = OldIndexBuffer[i];
					int32 VoidCount  = Mask[OldVertIdx];
					int32 NewVertIdx = OldVertIdx - VoidCount;

					checkSlow(NewVertIdx > -1);

					IndexBuffer[i] = NewVertIdx;
				}


				// Clean up the temporary 
				delete[] OldVertexBuffer;
				delete[] OldIndexBuffer;
			}
			delete[] Mask;
		}


		uint32* IndexBuffer;
		SimpVertexType* VertexBuffer;


	private:

		TSkinnedSkeletalMesh(const TSkinnedSkeletalMesh&);

		int32 NumTris;
		int32 NumVerts;
		int32 TexCoordNum;
	};

	typedef TSkinnedSkeletalMesh<SkeletalSimplifier::MeshVertType>  FSkinnedSkeletalMesh;

}