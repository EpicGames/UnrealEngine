// Copyright Epic Games, Inc. All Rights Reserved.

#include "SkeletalSimplifier.h"


const int32 SkeletalSimplifier::LinearAlgebra::SymmetricMatrix::Mapping[9] = { 0, 1, 2,
                                                                               1, 3, 4,
                                                                               2, 4, 5 };



//=============
// FMeshSimplifier
//=============
SkeletalSimplifier::FMeshSimplifier::FMeshSimplifier(const MeshVertType* InSrcVerts, const uint32 InNumSrcVerts,
	const uint32* InSrcIndexes, const uint32 InNumSrcIndexes,
	const float CoAlignmentLimit, const float VolumeImportanceValue, const bool VolumeConservation, const bool bEnforceBoundaries, 
	const bool bMergeCoincidentVertBones, const bool bUseLegacyAttrGradient)
	:
	coAlignmentLimit(CoAlignmentLimit),
	VolumeImportance(VolumeImportanceValue),
	bPreserveVolume(VolumeConservation),
	bCheckBoneBoundaries(bEnforceBoundaries),
	bMergeBonesOnCoincidentVerts(bMergeCoincidentVertBones),
	bUseLegacyAttrGrad(bUseLegacyAttrGradient),
	MeshManager(InSrcVerts, InNumSrcVerts, InSrcIndexes, InNumSrcIndexes, bMergeCoincidentVertBones)
{

	// initialize the weights to be unit
	int32 NumBaseAttrs = MeshVertType::NumBaseAttributes();
	BasicAttrWeights.Reset();
	checkSlow(NumBaseAttrs == BasicAttrWeights.Num());

	for (int32 i = 0; i < MeshVertType::NumBaseAttributes(); ++i)
	{
		BasicAttrWeights.SetElement(i, 1.);
	}



	const int32 NumEdges = MeshManager.TotalNumEdges();

	// Allocate the edge collapse heap
	this->CollapseCostHeap.Resize(NumEdges, NumEdges);


	QuadricCache.RegisterMesh(MeshManager);
}



SkeletalSimplifier::FMeshSimplifier::~FMeshSimplifier()
{
}

void SkeletalSimplifier::FMeshSimplifier::SetBoundaryConstraintWeight(const double Weight)
{
	BoundaryConstraintWeight = Weight;
}

void SkeletalSimplifier::FMeshSimplifier::SetAttributeWeights(const DenseVecDType& Weights)
{
	// Make sure we have enough weights for the basic attrs.
	check(Weights.Num() == MeshVertType::BasicAttrContainerType::Size());

	BasicAttrWeights = Weights;
}

void SkeletalSimplifier::FMeshSimplifier::SetSparseAttributeWeights(const SparseWeightContainerType& SparseWeights)
{
	AdditionalAttrWeights = SparseWeights;
}


void SkeletalSimplifier::FMeshSimplifier::SetBoundaryLocked()
{
	MeshManager.FlagBoundary(ESimpElementFlags::SIMP_LOCKED);

}

void SkeletalSimplifier::FMeshSimplifier::SetColorEdgeLocked(float ColorDistThreshold)
{

	auto IsDifferntColor = [ColorDistThreshold](const SimpVertType* ASimpVert, const SimpVertType* BSimpVert)->bool
	{
		bool Result = true;
		if (ASimpVert!=nullptr && BSimpVert != nullptr)
		{ 
			const FLinearColor& AColor = ASimpVert->vert.BasicAttributes.Color;
			const FLinearColor& BColor = BSimpVert->vert.BasicAttributes.Color;

		
			Result = (FLinearColor::Dist(AColor, BColor) > ColorDistThreshold);
		}
		return Result;
	};

	MeshManager.FlagEdges(IsDifferntColor, ESimpElementFlags::SIMP_LOCKED);


}

void SkeletalSimplifier::FMeshSimplifier::SetBoxCornersLocked()
{

	MeshManager.FlagBoxCorners(ESimpElementFlags::SIMP_LOCKED);
}


void SkeletalSimplifier::FMeshSimplifier::InitCosts()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SkeletalSimplifier::InitCosts);

	const int32 NumEdges = MeshManager.TotalNumEdges();
	for (int i = 0; i < NumEdges; ++i)
	{
		SimpEdgeType* edgePtr = MeshManager.GetEdgePtr(i);
		double cost = ComputeEdgeCollapseCost(edgePtr);
		checkSlow(FMath::IsFinite(cost));
		CollapseCostHeap.Add(cost, i);
	}
}



FVector SkeletalSimplifier::FMeshSimplifier::ComputeEdgeCollapseVertsPos(SimpEdgeType* edge, EdgeUpdateTupleArray& EdgeAndNewVertArray, TArray< WedgeQuadricType, TInlineAllocator<16> >& wedgeQuadricArray, Quadrics::FEdgeQuadric& edgeQuadric)
{

	typedef SkeletalSimplifier::Quadrics::TQuadricOptimizer<WedgeQuadricType>      OptimizerType;

	// The newVerts and quadrics are going to be populated by this function.

	checkSlow(EdgeAndNewVertArray.Num() == 0);
	checkSlow(wedgeQuadricArray.Num() == 0);

	edgeQuadric.Zero();

	SimpEdgeType* e;
	SimpVertType* v;

	OptimizerType  Optimizer;

	//LockVertFlags(SIMP_MARK1);

	edge->v0->EnableFlagsGroup(SIMP_MARK1);
	edge->v1->EnableFlagsGroup(SIMP_MARK1);

	// add edges
	e = edge;
	do {
		checkSlow(e == MeshManager.FindEdge(e->v0, e->v1));
		checkSlow(e->v0->adjTris.Num() > 0);
		checkSlow(e->v1->adjTris.Num() > 0);

		EdgeAndNewVertArray.Emplace(e->v0, e->v1, e->v1->vert);

		WedgeQuadricType quadric = GetWedgeQuadric(e->v0);
		                 quadric += GetWedgeQuadric(e->v1);
		wedgeQuadricArray.Add(quadric);


		e->v0->DisableFlags(SIMP_MARK1);
		e->v1->DisableFlags(SIMP_MARK1);

		e = e->next;
	} while (e != edge);

	// add remainder verts
	v = edge->v0;
	do {
		if (v->TestFlags(SIMP_MARK1))
		{
			//newVerts.Add(v->vert);
			EdgeAndNewVertArray.Emplace(v, nullptr, v->vert);

			WedgeQuadricType quadric = GetWedgeQuadric(v);
			wedgeQuadricArray.Add(quadric);


			v->DisableFlags(SIMP_MARK1);
		}
		v = v->next;
	} while (v != edge->v0);

	v = edge->v1;
	do {
		if (v->TestFlags(SIMP_MARK1))
		{
			EdgeAndNewVertArray.Emplace(nullptr, v, v->vert);

			WedgeQuadricType quadric = GetWedgeQuadric(v);
			wedgeQuadricArray.Add(quadric);


			v->DisableFlags(SIMP_MARK1);
		}
		v = v->next;
	} while (v != edge->v1);



	checkSlow(wedgeQuadricArray.Num() <= 256);

	

	// Include EdgeQuadrics to try to keep UV seams 
	// and the like from deviating too much.
	// NB: the edge quadric will be non-zero only if the edge
	//     has a single side..
	
	v = edge->v0;
	do {
		// This includes an edge quadric for every single-sided edge that shares this vert
		edgeQuadric += GetEdgeQuadric(v);
		v = v->next;
	} while (v != edge->v0);

	v = edge->v1;
	do {
		// This includes an edge quadric for every single-sided edge that shares this vert
		edgeQuadric += GetEdgeQuadric(v);
		v = v->next;
	} while (v != edge->v1);

	// Add all the quadrics to the Optimizer
	Optimizer.AddEdgeQuadric(edgeQuadric);  // edge quadric
	for (uint32 q = 0, qMax = wedgeQuadricArray.Num(); q < qMax; ++q)
	{
		Optimizer.AddFaceQuadric(wedgeQuadricArray[q]); // vertex quadrics
	}

	// Compute the new location
	FVector newPos;
	{
		bool bLocked0 = edge->v0->TestFlags(SIMP_LOCKED);
		bool bLocked1 = edge->v1->TestFlags(SIMP_LOCKED);
		checkSlow( !( bLocked0 && bLocked1) ); // can't have both locked

		// find position
		if (bLocked0)
		{
			// v0 position
			newPos = (FVector)edge->v0->GetPos();
		}
		else if (bLocked1)
		{
			// v1 position
			newPos = (FVector)edge->v1->GetPos();
		}
		else
		{
			// optimal position
			LinearAlgebra::Vec3d OptimalPos;
			bool valid = Optimizer.Optimize(OptimalPos, bPreserveVolume, VolumeImportance);
			if (!valid)
			{
				// Couldn't find optimal so choose middle
				newPos = (FVector)(edge->v0->GetPos() + edge->v1->GetPos()) * 0.5f;
			}
			else
			{
				newPos[0] = OptimalPos[0]; 
				newPos[1] = OptimalPos[1]; 
				newPos[2] = OptimalPos[2];
			}
		}
	}

	return newPos;

}


void SkeletalSimplifier::FMeshSimplifier::ComputeEdgeCollapseVerts(SimpEdgeType* edge, EdgeUpdateTupleArray& EdgeAndNewVertArray)
{
	// The newVerts are going to be populated by this function.

	checkSlow(EdgeAndNewVertArray.Num() == 0);

	// Compute the quadrics for verts and compute the new vert location.
	Quadrics::FEdgeQuadric edgeQuadric;
	TArray< WedgeQuadricType, TInlineAllocator<16> > wedgeQuadricArray;

	ComputeEdgeCollapseVertsAndQuadrics(edge, EdgeAndNewVertArray, edgeQuadric, wedgeQuadricArray);
}

void SkeletalSimplifier::FMeshSimplifier::ComputeEdgeCollapseVertsAndFixBones(SimpEdgeType* edge, EdgeUpdateTupleArray& EdgeAndNewVertArray)
{
	ComputeEdgeCollapseVerts(edge, EdgeAndNewVertArray);

	// Positions of the two edges
	const SimpVertType* Vert0 = edge->v0;
	const SimpVertType* Vert1 = edge->v1;

	const FVector Pos0 = (FVector)Vert0->vert.GetPos();
	const FVector Pos1 = (FVector)Vert1->vert.GetPos();

	// Position of the collapsed vert

	const FVector CollapsedPos = (FVector)EdgeAndNewVertArray[0].Get<2>().GetPos();

	// Find edge endpoint that is closest to the collapsed vert location.

	const float DstSqr0 = FVector::DistSquared(CollapsedPos, Pos0);
	const float DstSqr1 = FVector::DistSquared(CollapsedPos, Pos1);

	const auto& ClosestSimpliferVert = (DstSqr1 < DstSqr0) ? Vert1->vert : Vert0->vert;

	const auto& SrcBones         = ClosestSimpliferVert.GetSparseBones();
	const int32 ClosestSrcVertIndex  = ClosestSimpliferVert.ClosestSrcVertIndex;
	const int32 NumNewVerts      = EdgeAndNewVertArray.Num();

	for (int32 i = 0; i < NumNewVerts; ++i)
	{
		MeshVertType& simpVert   = EdgeAndNewVertArray[i].Get<2>();
		simpVert.SparseBones     = SrcBones;
		simpVert.ClosestSrcVertIndex = ClosestSrcVertIndex;
	}

}


double SkeletalSimplifier::FMeshSimplifier::ComputeEdgeCollapseVertsAndCost(SimpEdgeType* edge, EdgeUpdateTupleArray& EdgeAndNewVertArray)
{
	// The newVerts are going to be populated by this function.

	checkSlow(EdgeAndNewVertArray.Num() == 0);

	// Compute the quadrics for verts and compute the new vert location.
	Quadrics::FEdgeQuadric edgeQuadric;
	TArray< WedgeQuadricType, TInlineAllocator<16> > WedgeQuadricArray;

	ComputeEdgeCollapseVertsAndQuadrics(edge, EdgeAndNewVertArray, edgeQuadric, WedgeQuadricArray);

	// All the MeshVerts share the same location - just use the first for this
	double cost = edgeQuadric.Evaluate(EdgeAndNewVertArray[0].Get<2>().GetPos());

	// accumulate the attribute quadrics values.
	for (int i = 0; i < WedgeQuadricArray.Num(); i++)
	{
		MeshVertType& simpVert = EdgeAndNewVertArray[i].Get<2>();

		// sum cost of new verts 
		cost += WedgeQuadricArray[i].Evaluate(simpVert, BasicAttrWeights, AdditionalAttrWeights);
	}

	return cost;
}



float SkeletalSimplifier::FMeshSimplifier::CalculateNormalShift(const SimpTriType& tri, const SimpVertType* oldVert, const FVector& Pos) const 
{
	uint32 k;
	if ( oldVert== tri.verts[0])
		k = 0;
	else if (oldVert == tri.verts[1])
		k = 1;
	else
		k = 2;

	const FVector& v0 = (FVector)tri.verts[k]->GetPos();
	const FVector& v1 = (FVector)tri.verts[k = (1 << k) & 3]->GetPos();
	const FVector& v2 = (FVector)tri.verts[k = (1 << k) & 3]->GetPos();

	const FVector d21 = v2 - v1;
	const FVector d01 = v0 - v1;
	const FVector dp1 = Pos - v1;

	// the current face normal
	FVector n0 = FVector::CrossProduct(d01, d21);

	// face normal when you move oldVert to Pos
	FVector n1 = FVector::CrossProduct(dp1, d21);

	bool bValid = n0.Normalize();
	bValid = bValid && n1.Normalize();

	return (bValid) ? FVector::DotProduct(n0, n1) : 1.f;
}

double SkeletalSimplifier::FMeshSimplifier::ComputeEdgeCollapseCost(SimpEdgeType* edge)
{


	if (edge->v0->TestFlags(SIMP_LOCKED) && edge->v1->TestFlags(SIMP_LOCKED))
	{
		return FLT_MAX;
	}


	// New verts that replace the edges in this edge group.

	EdgeUpdateTupleArray EdgeAndNewVertArray;
	double cost = ComputeEdgeCollapseVertsAndCost(edge, EdgeAndNewVertArray);

	// All the new verts share the same location, but will have different attributes.

	const FVector newPos = (FVector)EdgeAndNewVertArray[0].Get<2>().GetPos();

	// add penalties
	// the below penalty code works with groups so no need to worry about remainder verts

	SimpVertType* u = edge->v0;
	SimpVertType* v = edge->v1;
	SimpVertType* vert;

	double penalty = 0.;

	{
		const double P = this->DegreePenalty;
		const uint32 L = this->DegreeLimit;

		//const int degreeLimit = 24;
		//const float degreePenalty = 100.0f;

		// Degrees =  number of triangles that share u + number of triangles that share v.
		uint32 Degrees = MeshManager.GetDegree(*u);
		Degrees       += MeshManager.GetDegree(*v);


		if (Degrees > L)
		{
			penalty += P * (Degrees - L);
		}
	}

	if (bCheckBoneBoundaries)
	// Penalty to select against bone boundaries
	{
		const double BadBonePenalty = invalidPenalty;

		bool bIsBoneBoundary = false;

		// Test all the edges in this edge group to see 
		// if the collapse of one of them violates the bone criteria

		SimpEdgeType* e = edge;
	
		do {

			// Penalty for collapsing across a bone boundary.
			const auto& UBones = e->v0->vert.GetSparseBones();
			const auto& VBones = e->v1->vert.GetSparseBones();

			auto UBoneIDIter = UBones.GetData().CreateConstIterator();
			auto VBoneIDIter = VBones.GetData().CreateConstIterator();

			if (UBoneIDIter && VBoneIDIter)
			{
				int32 ULeadingBone = UBoneIDIter.Key();
				int32 VLeadingBone = VBoneIDIter.Key();

				if (ULeadingBone  != VLeadingBone)
				{
					bIsBoneBoundary = true;
				}
			}
#if 0
			if (UBones.Num() != VBones.Num())
			{
				penalty += BadBonePenalty;
			}
			else
			{
				auto UBoneIDIter = UBones.GetData().CreateConstIterator();
				auto VBoneIDIter = VBones.GetData().CreateConstIterator();
				
				for (; UBoneIDIter && VBoneIDIter; ++UBoneIDIter, ++VBoneIDIter)
				{
					int32 ULeadingBone = UBoneIDIter.Key();
					int32 VLeadingBone = VBoneIDIter.Key();
					if (ULeadingBone != VLeadingBone)  // check the bone id 
					{
						penalty += BadBonePenalty;
					}
				}
			}
#endif			
			e = e->next;
		} while (e != edge && bIsBoneBoundary == false);

		if (bIsBoneBoundary)
		{
			penalty += BadBonePenalty;
		}

		#if (0)
		// Do all the verts in each vert group have the same bones?
		{

			auto VertexGroupBoneValidator = [BadBonePenalty](SimpVertType* v)->double
			{
				bool bValid = true;
				SimpVertType* vTmp = v;
				const auto& BoneArray = v->vert.GetSparseBones();
				const int32 NumBones = BoneArray.Num();

				while (vTmp->next != v)
				{
					vTmp = vTmp->next;
					const auto& OtherBoneArray = vTmp->vert.GetSparseBones();
					if (NumBones != OtherBoneArray.Num())
					{
						bValid = false;;
					}
					else
					{
						// Test that the bones are in the same order
						auto UBoneIDIter = BoneArray.GetData().CreateConstIterator();
						auto VBoneIDIter = OtherBoneArray.GetData().CreateConstIterator();

						for (; UBoneIDIter && VBoneIDIter; ++UBoneIDIter, ++VBoneIDIter)
						{
							int32 ULeadingBone = UBoneIDIter.Key();
							int32 VLeadingBone = VBoneIDIter.Key();
							if (ULeadingBone != VLeadingBone)  // check the bone id 
							{
								bValid = false;
							}
						}
					}
				}

				return (bValid)? 0. : BadBonePenalty;

			};

			penalty += VertexGroupBoneValidator(e->v0);
			penalty += VertexGroupBoneValidator(e->v1);

		}
		#endif

	}



	{

		const double penaltyToPreventEdgeFolding = invalidPenalty;
		const double penaltyAgainstCurvatureChange = 0.25 * invalidPenalty;

		// Keep track of the max specialized weight associated with a split vert in this group.
		float SpecialWeight = 0.f;

		v->EnableAdjTriFlagsGroup(SIMP_MARK1);

		// u
		vert = u;
		do {
			SpecialWeight = FMath::Max(SpecialWeight, vert->vert.SpecializedWeight);
			for (TriIterator i = vert->adjTris.Begin(); i != vert->adjTris.End(); ++i)
			{
				SimpTriType* tri = *i;
				if (!tri->TestFlags(SIMP_MARK1))
				{
					
#if 1
					if (!tri->ReplaceVertexIsValid(vert, (FVector3f)newPos))
					{
						penalty += penaltyToPreventEdgeFolding;
					}
#else
					// dot product of face normal with the face normal that results from moving the vert to new vert.
					float NormDotNorm = CalculateNormalShift(*tri, vert, newPos);
					if (NormDotNorm < 0.f)
					{
						penalty += penaltyToPreventEdgeFolding;

					}
					else if (NormDotNorm < coAlignmentLimit)
					{
						penalty += penaltyAgainstCurvatureChange;
					}
#endif
					//penalty += tri->ReplaceVertexIsValid(vert, newPos, minDotProduct) ? 0.0f : penaltyToPreventEdgeFolding;
				}
				tri->DisableFlags(SIMP_MARK1);
			}
			vert = vert->next;
		} while (vert != u);

		// v
		vert = v;
		do {
			SpecialWeight = FMath::Max(SpecialWeight, vert->vert.SpecializedWeight);
			for (TriIterator i = vert->adjTris.Begin(); i != vert->adjTris.End(); ++i)
			{
				SimpTriType* tri = *i;
				if (tri->TestFlags(SIMP_MARK1))
				{
#if 1
					if (!tri->ReplaceVertexIsValid(vert, (FVector3f)newPos))
					{
						penalty += penaltyToPreventEdgeFolding;
					}
#else
					// dot product of face normal with the face normal that results from moving the vert to new vert.
					float NormDotNorm = CalculateNormalShift(*tri, vert, newPos);
					if (NormDotNorm < 0.f)
					{
						penalty += penaltyToPreventEdgeFolding;

					}
					else if (NormDotNorm < coAlignmentLimit)
					{
						penalty += penaltyAgainstCurvatureChange;
					}
					//penalty += tri->ReplaceVertexIsValid(vert, newPos, minDotProduct) ? 0.0f : penaltyToPreventEdgeFolding;
#endif
				}
				tri->DisableFlags(SIMP_MARK1);
			}
			vert = vert->next;
		} while (vert != v);

		penalty += SpecialWeight;
	}

	return  cost + penalty;
}


int32 SkeletalSimplifier::FMeshSimplifier::CountDegenerates() const
{
	return MeshManager.CountDegeneratesTris();
}


void SkeletalSimplifier::FMeshSimplifier::OutputMesh(MeshVertType* verts, uint32* indexes, bool bWeldVtxColorAttrs, TArray<int32>* LockedVerts)
{

	if(bMergeBonesOnCoincidentVerts)
	{
		// Fix-up to make sure that verts that share the same location (e.g. UV boundaries) have the same bone weights
		// (otherwise we will get cracks when the characters animate)
		// Get the vert groups that share the same locations.

		VertPtrArray CoincidentVertGroups;
		MeshManager.GetCoincidentVertGroups(CoincidentVertGroups);

		const int32 NumCoincidentVertGroups = CoincidentVertGroups.Num();
 

		// Make sure all the bones are the same
		for (int32 i = 0; i < NumCoincidentVertGroups; ++i)
		{
			SimpVertType* HeadVert = CoincidentVertGroups[i];
			const auto& HeadSparseBones = HeadVert->vert.SparseBones;
			SimpVertType* tmp = HeadVert->next;
			while (tmp != HeadVert)
			{
				tmp->vert.SparseBones = HeadSparseBones;
				tmp = tmp->next;
			}

		}

	}


	if (bWeldVtxColorAttrs)
	{
		MeshManager.WeldNonSplitBasicAttributes(FSimplifierMeshManager::EVtxElementWeld::Color);
	}

	MeshManager.OutputMesh(verts, indexes, LockedVerts);

}
