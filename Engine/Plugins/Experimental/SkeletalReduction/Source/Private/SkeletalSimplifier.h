// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/BinaryHeap.h"
#include "SkeletalSimplifierQuadricCache.h"


namespace SkeletalSimplifier
{


	/**
	* Simple terminator class.  This could be expanded on to include an interrupter to allow the user
	* to halt execution.
	*/
	class FSimplifierTerminatorBase
	{
	public:
		FSimplifierTerminatorBase(int32 MinTri, int32 MinVert, float MaxCost)
			: MaxFeatureCost(MaxCost)
			, MinTriNumToRetain(MinTri)
			, MinVertNumToRetain(MinVert)
		{}

		// return true if the simplifier should terminate.
		inline bool operator()(const int32 TriNum, const int32 VertNum, const float SqrError)
		{
			if (TriNum < MinTriNumToRetain || VertNum < MinVertNumToRetain || SqrError > MaxFeatureCost)
			{
				return true;
			}
			return false;
		}


		float MaxFeatureCost;
		int32 MinTriNumToRetain;
		int32 MinVertNumToRetain;

	};

	/**
	* Termination criterion for  simplifier
	*/
	class FSimplifierTerminator : public FSimplifierTerminatorBase
	{
	public:
		FSimplifierTerminator(uint32 MinTri, uint32 MaxTri, uint32 MinVert, uint32 MaxVert, float MaxCost, float MaxDist)
			: FSimplifierTerminatorBase(MinTri, MinVert, MaxCost)
			, MaxTriNumToRetain(MaxTri)
			, MaxVertNumToRetain(MaxVert)
			, MaxDistance(MaxDist)
		{}

		// return true if the simplifier should terminate.
		inline bool operator()(const uint32 TriNum, const uint32 VertNum, const float SqrError)
		{
			if (FSimplifierTerminatorBase::operator()(TriNum, VertNum, SqrError) && TriNum < MaxTriNumToRetain && VertNum < MaxVertNumToRetain)
			{
				return true;
			}
			else
			{
				return false;
			}
		}


		uint32 MaxTriNumToRetain;
		uint32 MaxVertNumToRetain;
		float MaxDistance;

	};

	/**
	* Class to allow a single weight value to be applied to all the sparse attributes.
	*/
	class UniformWeights
	{
	public:
		UniformWeights() : Weight(0.)
		{}
		UniformWeights(double w) : Weight(w)
		{}

		UniformWeights(const UniformWeights& other) : Weight(other.Weight) {}

		double GetElement(int32) const { return Weight; }

		UniformWeights& operator=(const UniformWeights& other)
		{
			Weight = other.Weight;
			return *this;
		}

	private:
		double Weight;

	};

	/**
	*  The core simplifier.  This does all the work
	*
	*/
	class FMeshSimplifier
	{
		typedef typename SimpVertType::TriIterator	                         TriIterator;
		typedef Quadrics::TFaceQuadric<MeshVertType, UniformWeights>         WedgeQuadricType;
		typedef TQuadricCache<WedgeQuadricType>                              QuadricCacheType;
		typedef typename MeshVertType::BasicAttrContainerType                BasicAttrContainerType;



		typedef typename MeshVertType::AttrContainerType                     AttrContainerType;

		typedef TTuple<SimpVertType*, SimpVertType*, MeshVertType>           EdgeUpdateTuple;
		typedef TArray<EdgeUpdateTuple, TInlineAllocator<16> >               EdgeUpdateTupleArray;

		typedef TArray<SimpEdgeType*, TInlineAllocator<32> >                 EdgePtrArray;
		typedef TArray<SimpTriType*, TInlineAllocator<16> >                  TriPtrArray;
		typedef TArray<SimpVertType*, TInlineAllocator<16> >                 VertPtrArray;


	public:
		typedef typename BasicAttrContainerType::DenseVecDType               DenseVecDType;
		typedef typename BasicAttrContainerType::WeightArrayType             WeightArrayType;
		typedef typename WedgeQuadricType::SparseWeightContainerType         SparseWeightContainerType;


	public:

		FMeshSimplifier(const MeshVertType* InSrcVerts, const uint32 InNumSrcVerts,
			const uint32* InSrcIndexes, const uint32 InNumSrcIndexes,
			const float CoAlignmentLimit,
			const float VolumeImportance,
			const bool bVolumePreservation,
			const bool bEnforceBoundaries,
			const bool bMergeCoincidentVertBones,
			const bool bUseLegacyAttrGradient);

		~FMeshSimplifier();


		/**
		* Weight used to insure that when a boundary edge collapses, the resulting vertex wont be far from that edge.
		*/
		void                SetBoundaryConstraintWeight(const double Weight);

		/**
		* Set Quadric weights for the standard attributes ( the attributes that are stored in TBasicVertexAttrs<NumTexCoords> )
		* The order of the weights corresponds to the order of the attributes :
		*  Normal (3 weights), Tangent (3 weights ), BiTangent (3 weights), Color (3 weights), TextureCoords( 2 * NumTexCoords weights).
		*/
		void				SetAttributeWeights(const DenseVecDType& Weights);

		/**
		* Set Quadric weights for sparse attributes
		*/
		void                SetSparseAttributeWeights(const SparseWeightContainerType& SparseWeights);

		/**
		* Lock mesh boundary edges to prevent simplification by using the vertex tag ESimpElementFlags::SIMP_LOCKED
		*/
		void				SetBoundaryLocked();

		/**
		* Lock edges that connect different colors to prevent simplification by using the vertex tag ESimpElementFlags::SIMP_LOCKED
		*/
		void                SetColorEdgeLocked(float Threshold = 1.e-3);

		/**
		* Lock vertices at corners of very simple 8-vert boxes.
		*/
		void                SetBoxCornersLocked();

		/**
		* Simplify the mesh.  Termination of the simplification is controlled by the TiminationCriterion class
		* The terminator must implement
		*
		*             struct Terminator
		*             {
		*                inline bool operator()(const int32 TriNum, const float SqrError);
		*             };
		*/
		template <typename TerminationCriterionType>
		float				SimplifyMesh(TerminationCriterionType TerminationCriterion);

		/**
		* The number of verts of the mesh once simplified  (or at any stage)
		*/
		int					GetNumVerts() const { return MeshManager.ReducedNumVerts; }

		/**
		* The number of tris in the mesh once simplified (or at any stage)
		*/
		int					GetNumTris() const { return MeshManager.ReducedNumTris; }

		/**
		*  The fraction of non-manifold edges (edges with more than 2 adjacent tris).
		*/
		float               FractionNonManifoldEdges() { return MeshManager.FractionNonManifoldEdges(); }

		/**
		* Export a copy of the simplified mesh.
		*
		* @param Verts                 - pointer to an array to populate.  Should be least GetNumVerts() in size
		* @param Indexes               - pointer to index buffer to populate.  Should be at least 3 * GetNumTris() in size
		* @param  bWeldVtxColorAttrs   - Weld verts attributes (colors) that may have been artificially split by the reduction algorithm.
		*                                note: this could be extended to the other attributes (e.g. normals, uvs )  
		* @param LockedVerts           - optional pointer to array.  On return the array will hold the indices of any locked verts.
		*/
		void				OutputMesh(MeshVertType* Verts, uint32* Indexes, bool bWeldVtxColorAttrs = true, TArray<int32>* LockedVerts = NULL);


	protected:

		// Populate the cost heap by evaluating the collapse cost of each edge.
		// NB: must be called before simplification
		void				InitCosts();

		// returns the dot product between current face normal and the face normal after vertToUpdate has been moved to newPos
		float               CalculateNormalShift(const SimpTriType& tri, const SimpVertType* vertToUpdate, const FVector& newPos) const;

		void				ComputeEdgeCollapseVerts(SimpEdgeType* edge, EdgeUpdateTupleArray& newVerts);
		void				ComputeEdgeCollapseVertsAndFixBones(SimpEdgeType* edge, EdgeUpdateTupleArray& newVerts);


		double				ComputeEdgeCollapseVertsAndCost(SimpEdgeType* edge, EdgeUpdateTupleArray& newVerts);
		FVector             ComputeEdgeCollapseVertsPos(SimpEdgeType* edge, EdgeUpdateTupleArray& newVerts, TArray< WedgeQuadricType, TInlineAllocator<16> >& quadrics, Quadrics::FEdgeQuadric& edgeQuadric);

		void                ComputeEdgeCollapseVertsAndQuadrics(SimpEdgeType* edge, EdgeUpdateTupleArray& newVerts, Quadrics::FEdgeQuadric& newEdgeQuadric, TArray< WedgeQuadricType, TInlineAllocator<16> >& newQuadrics);

		double				ComputeEdgeCollapseCost(SimpEdgeType* edge);

		// Quadric based on attributes and position (see section 4 in Hoppe's "New Quadric Metric for Simplifying Meshes.")
		WedgeQuadricType    GetWedgeQuadric(SimpVertType* v);

		// Quadric based on sharp edges. (See section 5 in Hoppe's "New Quadric Metric for Simplifying Meshes.")
		// This includes an edge quadric for every single-sided edge that shares this vert
		Quadrics::FEdgeQuadric		GetEdgeQuadric(SimpVertType* v);

		void                DirtyTriQuadricCache(TriPtrArray& DirtyTri);
		void                DirtyVertAndEdgeQuadricsCache(VertPtrArray& DirtyVerts);


		// 
		void                UpdateEdgeCollapseCost(EdgePtrArray& DirtyEdges);

		int32               CountDegenerates() const; // Included for testing.


		// Snap the interpolated vertex color in the third tuple element to the closest of the other two
		void                SnapToClosestSourceVertexColor(EdgeUpdateTuple& UpdateTuple);

		// Returns the closer (AColor or BColor) from the SampleColor
		const FLinearColor& SnapToClosestColor(const FLinearColor& AColor, const FLinearColor& BColor, const FLinearColor& SampleColor) const;


	protected:

		// --- Weights for quadric simplification

		// The weights for the standard attributes
		DenseVecDType             BasicAttrWeights;

		// The weights for the bones
		SparseWeightContainerType AdditionalAttrWeights;


		// --- Magic numbers that penalize undesirable simplifications.

		// prevent high valence verts.
		uint32               DegreeLimit = 24;
		double               DegreePenalty = 100.0f;

		// prevent edge folding 
		double               invalidPenalty = 1.e6;

		// critical angle by which the normal of a triangle is allowed to changed during a collapse.
		double               coAlignmentLimit = .0871557f;

		double               VolumeImportance;

		bool                 bPreserveVolume;
		bool                 bCheckBoneBoundaries;
		bool                 bMergeBonesOnCoincidentVerts;
		double               BoundaryConstraintWeight = 256.;



		// --- End Magic numbers

		// forces the simplifier to use the old attribute interpolation gradient method.  Needed for backward compatibilty with certain cloth assets
		bool bUseLegacyAttrGrad;


		// Heap that maps edges to the cost of collapse
		FBinaryHeap<double>		CollapseCostHeap;

		// Manage the quadrics
		// Linear arrays that map quadrics to vertices, faces and edges.
		QuadricCacheType        QuadricCache;

		// The mesh manager - holds a mesh that supports the topology queries we need and the collapse methods.
		FSimplifierMeshManager   MeshManager;


	private:
		FMeshSimplifier();

	};


	FORCEINLINE void  FMeshSimplifier::DirtyTriQuadricCache(TriPtrArray& DirtyTriArray)
	{
		for (SimpTriType* tri : DirtyTriArray)
		{
			//if (tri->TestFlags(SIMP_REMOVED)) continue;

			QuadricCache.DirtyTriQuadric(tri);
		}
	}
	FORCEINLINE void  FMeshSimplifier::DirtyVertAndEdgeQuadricsCache(VertPtrArray& DirtyVertArray)
	{
		for (SimpVertType* vert : DirtyVertArray)
		{
			//if (vert->TestFlags(SIMP_REMOVED)) continue;

			const uint32 VertIdx = MeshManager.GetVertIndex(vert);
			QuadricCache.DirtyVertQuadric(VertIdx);
			QuadricCache.DirtyEdgeQuadric(VertIdx);
		}
	}

	FORCEINLINE  FMeshSimplifier::WedgeQuadricType FMeshSimplifier::GetWedgeQuadric(SimpVertType* v)
	{

		const auto 	TriQuadricFatory = [this](const SimpTriType& tri)->FMeshSimplifier::WedgeQuadricType
		{
			return FMeshSimplifier::WedgeQuadricType(
				tri.verts[0]->vert, tri.verts[1]->vert, tri.verts[2]->vert,
				this->BasicAttrWeights, this->AdditionalAttrWeights, bUseLegacyAttrGrad);
		};

		return QuadricCache.GetWedgeQuadric(v, TriQuadricFatory);
	}


	FORCEINLINE Quadrics::FEdgeQuadric FMeshSimplifier::GetEdgeQuadric(SimpVertType* v)
	{
		const double Weight = BoundaryConstraintWeight;
		const auto EdgeQuadricFactory = [this, Weight](const FVector& Pos0, const FVector& Pos1, const FVector& Normal)->Quadrics::FEdgeQuadric
		{
			return Quadrics::FEdgeQuadric(Pos0, Pos1, Normal, Weight);
		};

		Quadrics::FEdgeQuadric Quadric = QuadricCache.GetEdgeQuadric(v, EdgeQuadricFactory);

		return Quadric;
	}



	FORCEINLINE void FMeshSimplifier::ComputeEdgeCollapseVertsAndQuadrics(SimpEdgeType* edge,
		EdgeUpdateTupleArray& EdgeAndNewVertArray,
		Quadrics::FEdgeQuadric& newEdgeQuadric,
		TArray< WedgeQuadricType, TInlineAllocator<16> >& newWedgeQuadrics)
	{
		FVector newPos = ComputeEdgeCollapseVertsPos(edge, EdgeAndNewVertArray, newWedgeQuadrics, newEdgeQuadric);

		// update all the collapsed verts with the new location
		for (int i = 0; i < newWedgeQuadrics.Num(); ++i)
		{
			MeshVertType& simpVert = EdgeAndNewVertArray[i].Get<2>();
			simpVert.GetPos() = (FVector3f)newPos;

			// If the area is non-degenerate:  by default the elements in newVertPair are copies of the original v1 verts.
			if (newWedgeQuadrics[i].TotalArea() > 1.e-6)
			{
				// calculate vert attributes from the new position
				newWedgeQuadrics[i].CalcAttributes(simpVert, BasicAttrWeights, AdditionalAttrWeights);

				simpVert.Correct();
			}
		}


	}

	FORCEINLINE const FLinearColor& FMeshSimplifier::SnapToClosestColor(const FLinearColor& AColor, const FLinearColor& BColor, const FLinearColor& SampleColor) const
	{
		// compute the squared distance in linear space - ignoring the alpha channel.
		float ADistSqrd = (AColor.R - SampleColor.R)*(AColor.R - SampleColor.R) + (AColor.G - SampleColor.G)*(AColor.G - SampleColor.G) + (AColor.B - SampleColor.B)*(AColor.B - SampleColor.B);
		float BDistSqrt = (BColor.R - SampleColor.R)*(BColor.R - SampleColor.R) + (BColor.G - SampleColor.G)*(BColor.G - SampleColor.G) + (BColor.B - SampleColor.B)*(BColor.B - SampleColor.B);

		// select the closest
		return (ADistSqrd > BDistSqrt) ? BColor : AColor;

	}

	FORCEINLINE void FMeshSimplifier::SnapToClosestSourceVertexColor(EdgeUpdateTuple& UpdateTuple)
	{
		MeshVertType&  VertAttributes = UpdateTuple.Get<2>();
		FLinearColor& InterpolatedColor = VertAttributes.BasicAttributes.Color;

		const SimpVertType* AVtxPtr = UpdateTuple.Get<0>();
		const SimpVertType* BVtxPtr = UpdateTuple.Get<1>();

		if (AVtxPtr != nullptr && BVtxPtr != nullptr)
		{
			// update to the closest source color.  
			InterpolatedColor = SnapToClosestColor(AVtxPtr->vert.BasicAttributes.Color, BVtxPtr->vert.BasicAttributes.Color, InterpolatedColor);
		}
		
	}

	FORCEINLINE void FMeshSimplifier::UpdateEdgeCollapseCost(EdgePtrArray& DirtyEdges)
	{
		uint32 NumEdges = DirtyEdges.Num();
		// update edges
		for (uint32 i = 0; i < NumEdges; i++)
		{
			SimpEdgeType* edge = DirtyEdges[i];

			if (edge->TestFlags(SIMP_REMOVED))
				continue;

			double cost = ComputeEdgeCollapseCost(edge);

			SimpEdgeType* e = edge;
			do {
				uint32 EdgeIndex = MeshManager.GetEdgeIndex(e);
				if (CollapseCostHeap.IsPresent(EdgeIndex))
				{
					CollapseCostHeap.Update(cost, EdgeIndex);
				}
				e = e->next;
			} while (e != edge);
		}
	}

	template< typename TerminationCriterionType>
	float FMeshSimplifier::SimplifyMesh(TerminationCriterionType TerminationCriterion)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FMeshSimplifier::SimplifyMesh);

		// Build the cost heap
		InitCosts();

		// Do the distance check?
		const bool bCheckDistance = (TerminationCriterion.MaxDistance < FLT_MAX);

		TriPtrArray  DirtyTris;
		VertPtrArray DirtyVerts;
		EdgePtrArray DirtyEdges;

		double maxError = 0.0f;
		float  distError = 0.0f;

		while (CollapseCostHeap.Num() > 0)
		{
		

			// get the next vertex to collapse
			uint32 TopIndex = CollapseCostHeap.Top();

			const double error = CollapseCostHeap.GetKey(TopIndex);

			// Check for termination.
			{
				const int32 numTris  = MeshManager.ReducedNumTris;
				const int32 numVerts = MeshManager.ReducedNumVerts;
				if (TerminationCriterion(numTris, numVerts, error) || distError > TerminationCriterion.MaxDistance)
				{
					break;
				}
			}

			maxError = FMath::Max(maxError, error);

			CollapseCostHeap.Pop();

			// Pointer to the candidate edge (link list) for collapse
			SimpEdgeType* TopEdgePtr = MeshManager.GetEdgePtr(TopIndex);
			checkSlow(TopEdgePtr);
			

			// Gather all the edges that are really in this group.
			EdgePtrArray CoincidentEdges;
			MeshManager.GetEdgesInGroup(*TopEdgePtr, CoincidentEdges);

			if (MeshManager.HasLockedVerts(TopEdgePtr) || MeshManager.IsLockedGroup(CoincidentEdges))
			{
				// this edge shouldn't be collapsed.
				continue;
			}

			// Before changing any of the mesh topology, we capture lists
			// of the tris, verts, and edges that may need new quadrics
			// computed to updated edge collapse values. 
			MeshManager.GetAdjacentTopology(*TopEdgePtr, DirtyTris, DirtyVerts, DirtyEdges);

			const int32 NumCoincidentEdges = CoincidentEdges.Num();

			// Remove any edges in from this group that happen to be degenerate (no adjacent triangles)
			// and capture the Idx of the dead edges.  Also removes the edges from the edge-index heap.
			FSimplifierMeshManager::IdxArray InvalidCostIdxArray;  // Keep track of the Idx to remove from the cost heap
			const int32 NumRemovedEdges = MeshManager.RemoveEdgeIfInvalid(CoincidentEdges, InvalidCostIdxArray);
			
			// if none of the edges in this group were valid, just continue.
			if (NumCoincidentEdges == NumRemovedEdges)
			{
				continue;
			}

			// The representative edge in our edge group was removed.  Replace it with another from the group that is valid.
			if (MeshManager.IsRemoved(TopEdgePtr) || MeshManager.IsInvalid(TopEdgePtr))
			{
				// try to find a valid edge in the batch.. 
				for (SimpEdgeType* EPtr : CoincidentEdges)
				{
					if (EPtr && !MeshManager.IsRemoved(EPtr))
					{
						TopEdgePtr = EPtr;
						break;
					}
				}
			}

			// continue if no edge actually exists.
			if (MeshManager.IsRemoved(TopEdgePtr) || MeshManager.IsInvalid(TopEdgePtr))
			{
				continue;
			}

			// update Edge->v1 to new locations :  move verts to new verts
			{

				// Copy the  edge->v1->verts and update the location & attributes
				// capturing the corresponding SimpVert.
				// The VertexUpdate Array is built by adding 
				// 1) the two vertices for each edge
				// 2) Elements from TopEdge->v0 vert group not already added
				// 3) Elements from TopEdge->v1 vert group not already added
				EdgeUpdateTupleArray VertexUpdateArray;
				ComputeEdgeCollapseVertsAndFixBones(TopEdgePtr, VertexUpdateArray); // re-targets using closest bone
			
				// Compute the distance of the new vertex to each plane of the affected triangles. 
			
				if (bCheckDistance)
				{
					const FVector NewPos = (FVector)VertexUpdateArray[0].Get<2>().GetPos();

					float Dist = 0.f;
					for (SimpTriType* Tri : DirtyTris)
					{
						FVector TriNorm = (FVector)Tri->GetNormal();
						FVector PosToTri = NewPos - (FVector)Tri->verts[0]->GetPos();
						Dist = FMath::Max(FMath::Abs(FVector::DotProduct(TriNorm, PosToTri)), Dist);
					}

					distError = FMath::Max(distError, Dist);

				}

				
				// Update both verts in the mesh to have the new location and attribute values.

				for (int32 i = 0, Imax = VertexUpdateArray.Num(); i < Imax; ++i)
				{
					EdgeUpdateTuple&  EdgeUpdate = VertexUpdateArray[i];
					

					// New vertex attribute values for the collapsed vertex 
					const MeshVertType&  CollapsedAttributes = EdgeUpdate.Get<2>();

					// update the first edge vertex
					SimpVertType* VtxPtr = EdgeUpdate.Get<0>();
					if (VtxPtr != nullptr)
					{
						MeshManager.UpdateVertexAttributes(*VtxPtr, CollapsedAttributes);
					}

					// update the second edge vertex
					VtxPtr = EdgeUpdate.Get<1>();
					if (VtxPtr != nullptr)
					{
						MeshManager.UpdateVertexAttributes(*VtxPtr, CollapsedAttributes);
					}

				}
			}

			// This manages the complicated logic of making sure the attribute element IDs after the collapse
			// will be in the correct state for a collapse of of a split attribute or non-split attribute.  This applies
			// to the vertices in the edges, and the loose vertices that don't share a triangle
			// with the a vertex on the opposite end of the edge.

			MeshManager.UpdateVertexAttriuteIDs(CoincidentEdges);

			// collapse all edges by moving edge->v0 to edge->v1
			{
				
				// All positions and attributes should be fixed now, but we need to update the
				// mesh connectivity. 
				for (int i = 0; i < NumCoincidentEdges; ++i)
				{
					SimpEdgeType* EdgePtr = CoincidentEdges[i];

					bool bSkip = !EdgePtr || MeshManager.IsRemoved(EdgePtr); 

					if (bSkip) continue;

					// Collapse the edge, delete triangles that become degenerate
					// and update any edges that used to include v0 to include v1 now.
					// V1 acquires any lock from v0
					bool bCollapsed = MeshManager.CollapseEdge(EdgePtr, InvalidCostIdxArray);

					// this edge has been collapsed.  Remove it from the CoincidentEdges array
					if (bCollapsed)
					{
						CoincidentEdges[i] = NULL;
					}

					// NB: two edges in this group may have shared a single vertex. 
					// so the above collapse could have also collapsed other edges in the group.
					MeshManager.RemoveEdgeIfInvalid(CoincidentEdges, InvalidCostIdxArray);

				}

				// add v0 remainder verts to v1

				{
					// I'm not totally okay with this.. this adds all the v0 verts to the v1 group,
					// resulting in v1, v1', v1''.. v0, v0', ..
					// but doesn't change their Position()
					MeshManager.MergeGroups(TopEdgePtr->v0, TopEdgePtr->v1);  

					// if any of the verts in the group is locked, make them all locked
					MeshManager.PropagateFlag<SIMP_LOCKED>(*TopEdgePtr->v1);

					// prune any SIMP_REMOVED verts from the v1 link list.
					MeshManager.PruneVerts<SIMP_REMOVED>(*TopEdgePtr->v1);
				}
			}

			// Dirty the quadric cache 
			// NB: these early out if the objects are marked as removed.
			// i think that is too clever and should be changed so they
			// always dirty everything in the list.
			DirtyTriQuadricCache(DirtyTris);
			DirtyVertAndEdgeQuadricsCache(DirtyVerts);


			// If a dirty tri has zero area, remove it and tag as SIMP_REMOVED
			MeshManager.RemoveIfDegenerate(DirtyTris);

			
			// Tag verts that aren't part of tris ad SIMP_REMOVED
			// and remove them from the mesh vert groups.
			MeshManager.RemoveIfDegenerate(DirtyVerts); 

			// Remove edges with out verts and update
			// the cost & heap for all the remaining dirty edges.
			// this triggers updates of the tri and vert cached quadrics.
			MeshManager.RemoveIfDegenerate(DirtyEdges, InvalidCostIdxArray);


			// If an edge collapses on a triangle, the other two edges are now
			// one.. this accounts for that sort of thing.
			MeshManager.RebuildEdgeLinkLists(DirtyEdges);

			// Update the cost heap by removing the dead edges we just pruned.
			int32 NumRemoved = InvalidCostIdxArray.Num();
			for (int32 i = 0; i < NumRemoved; ++i)
			{
				CollapseCostHeap.Remove(InvalidCostIdxArray[i]);
			}
			// update the Quadric Caches associated with the dirty edges 
			// and update the heap.
			UpdateEdgeCollapseCost(DirtyEdges);

			DirtyTris.Reset();
			DirtyVerts.Reset();
			DirtyEdges.Reset();
		}

		// remove degenerate triangles
		// not sure why this happens
		int32 NumTrisIJustRemoved =
		MeshManager.RemoveDegenerateTris();

		// remove orphaned verts
		int32 NumVertsIJustRemoved = 
		MeshManager.RemoveDegenerateVerts();


		return (bCheckDistance) ? distError : maxError;
	}


}; // End namespace SkeletalSimplifier
