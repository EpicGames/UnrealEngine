// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "Math/Vector.h"
#include <cmath> // sqrt

namespace SkeletalSimplifier
{
	// Specialized sparse and dense vectors and matrices
	// with basic linear algebra functionality and some basic tools
	// needed for Quadratic calculation.
	namespace LinearAlgebra
	{

		

		//  Base class used in fixed-length linear storage of double precision data
		template <int32 SIZE, typename DerivedType>
		class ArrayBase
		{
		public:

			typedef double       ScalarType;

			typedef ScalarType   DataType;
			typedef ScalarType  MyStorageType[SIZE];

			ArrayBase()
			{
				checkSlow(SIZE > 0);
			}

			void Reset()
			{
				memset(Data, 0, sizeof(Data));
			}

			// Data access
			ScalarType& operator() (const int32 idx)
			{
				checkSlow(-1 < idx && idx < SIZE);
				return Data[idx];
			}


			const ScalarType& operator() (const int32 idx) const
			{
				checkSlow(-1 < idx && idx < SIZE);
				return Data[idx];
			}

			//  Direct access
			ScalarType& operator[] (const int32 idx) { return operator()(idx); }
			const ScalarType& operator[] (const int32 idx) const { return operator()(idx); }

			DerivedType& operator=(const DerivedType& other)
			{
				memcpy(Data, other.Data, sizeof(Data));// for (int i = 0; i < SIZE; ++i) Data[i] = other[i];
				return *static_cast<DerivedType*>(this);
			}


			// Vector addition / subtraction
			DerivedType& operator+= (const DerivedType& other)
			{
				for (int32 i = 0; i < SIZE; ++i) Data[i] += other[i];
				return *static_cast<DerivedType*>(this);
			}

			DerivedType& operator-= (const DerivedType& other)
			{
				for (int32 i = 0; i < SIZE; ++i) Data[i] -= other[i];
				return *static_cast<DerivedType*>(this);
			}

			DerivedType& operator*= (double Scalar)
			{
				for (int32 i = 0; i < SIZE; ++i) Data[i] *= Scalar;
				return *static_cast<DerivedType*>(this);
			}

			DerivedType operator+(const DerivedType& other) const
			{
				DerivedType Result(*this);
				Result += other;

				return Result;
			}

			DerivedType operator-(const DerivedType& other) const
			{
				DerivedType Result(*this);
				Result -= other;

				return Result;
			}

			// The number of elements
			 int32 Num() const  { return SIZE; }

			ScalarType L2NormSqr() const
			{
				ScalarType Result = 0.;

				for (int32 i = 0; i < SIZE; ++i)
				{
					Result += Data[i] * Data[i];
				}
				return Result;
			}

		protected:
			MyStorageType  Data;
		};


		class Vec3d : public ArrayBase<3, Vec3d>
		{
		public:
			typedef ArrayBase<3, Vec3d>    MyBase;

			/**
			* Default Construct as zero vector.
			*/
			Vec3d() 
			{
				// Zero
				Reset();
			}

			Vec3d(double x, double y, double z)
			{
				Data[0] = x;  Data[1] = y; Data[2] = z;
			}

			Vec3d(const MyBase& base) :
				MyBase(base)
			{}

			Vec3d(const FVector3f& fvec)
			{
				Data[0] = fvec[0]; Data[1] = fvec[1]; Data[2] = fvec[2];
			}

			Vec3d(const FVector3d& fvec)
			{
				Data[0] = fvec[0]; Data[1] = fvec[1]; Data[2] = fvec[2];
			}

			/**
			* Reset all the values in this vector to zero.
			*/
			void Zero()
			{
				Reset();
			}

			/**
			* Strict equality check.
			*/
			bool operator==(const Vec3d& other) const 
			{
				return (Data[0] == other[0] && Data[1] == other[1] && Data[2] == other[2]);
			}

			/**
			* Scale this vector
			*/
			Vec3d operator*(double Scalar)
			{
				Vec3d Result(*this);
				Result *= Scalar;
				return Result;
			}

			/**
			* The square of the geometric length of the vector.
			*/
			double LengthSqrd() const
			{
				return Data[0] * Data[0] + Data[1] * Data[1] + Data[2] * Data[2];
			}

			/**
			* DotProduct of this vector with another.
			*/
			double DotProduct(const Vec3d& other) const
			{
				return Data[0] * other[0] + Data[1] * other[1] + Data[2] * other[2];
			}
		};


		static Vec3d operator*(double Scalar, Vec3d B)
		{
			B *= Scalar;
			return B;
		}

		
	    /**
		* Rescale the vector to have magnitude one.
		* NB: this can fail if the magnitude of the source
		*     vector is less than 1.e-8
		*
		* @return true is the normalization succeeds 
		*/
		static bool NormalizeVector(Vec3d& Vect)
		{
			double LengthSqrd = Vect.LengthSqrd();
			double temp = std::sqrt(LengthSqrd);
			bool Success = (FMath::Abs(temp) > 1.e-8);
			if (Success)
			{
				temp = 1. / temp;

				Vect *= temp;
			}
			return Success;
		}

		/**
		* Computes the Cross Product of two vectors
		*  Cross = tmpA X tmpB
		* 
		* @param tmpA - the first vector
		* @param tmpB - the second vector
		*
		* @return the result
		*/
		static Vec3d CrossProduct(const Vec3d& tmpA, const Vec3d& tmpB)
		{
			Vec3d    Result( tmpA[1] * tmpB[2] - tmpA[2] * tmpB[1],
			             	 tmpA[2] * tmpB[0] - tmpA[0] * tmpB[2],
				             tmpA[0] * tmpB[1] - tmpA[1] * tmpB[0] );
			return Result;
		}



		// Double precision 3x3 symmetric matrix
		class SymmetricMatrix : public ArrayBase<6, SymmetricMatrix>
		{
		public:
			typedef ArrayBase<6, SymmetricMatrix>            MyBase;
			typedef typename MyBase::ScalarType              ScalarType;

			SymmetricMatrix() 
			{
				// Zero
				Reset();
			}

			/**
			* Constructor: takes the upper triangle part of the symmetric matrix
			*/
			SymmetricMatrix(double a11, double a12, double a13,
				                        double a22, double a23,
				                                    double a33) 
			{
				Data[0] = a11;  Data[1] = a12;  Data[2] = a13;
				                Data[3] = a22;  Data[4] = a23;
				                                Data[5] = a33;
			}

			SymmetricMatrix(const MyBase& base) :
				MyBase(base)
			{}

			/**
			* Accesses elements in the matrix using standard M(i,j) notation

			* @param i - the row  in [0,3) 
			* @param j - the column in [0, 3)
			*/
			SymmetricMatrix::ScalarType operator()(int32 i, int32 j) const
			{
				checkSlow(-1 < i && i < 3 && -1 < j && j < 3);
				const int32 Idx = Mapping[j + i * 3];
				return Data[Idx];
			}
			/**
			* Accesses elements in the matrix using standard M(i,j) notation

			* @param i - the row  in [0,3)
			* @param j - the column in [0, 3)
			*/
			SymmetricMatrix::ScalarType& operator()(int32 i, int32 j)
			{
				checkSlow(-1 < i && i < 3 && -1 < j && j < 3);
				const int32 Idx = Mapping[j + i * 3];
				return Data[Idx];
			}

			/**
			* Produce a new Vec3d that is the result of SymetricMatrix vector multiplication.
			* NB: this does M*v  not v * M
			*
			* @param Vect  - three dimensional double precision vector
			*
			* @return  Matrix * Vector.
			*/
			Vec3d operator* (const Vec3d& Vect) const
			{
				Vec3d Result(
					Vect[0] * Data[0] + Vect[1] * Data[1] + Vect[2] * Data[2],
					Vect[0] * Data[1] + Vect[1] * Data[3] + Vect[2] * Data[4],
					Vect[0] * Data[2] + Vect[1] * Data[4] + Vect[2] * Data[5]
				);

				return Result;
			}

			/**
			* Produce a new Symetric matrix that is the result of SymetricMatrix times SymetricMatrix
			* NB: ThisMatrix * Other
			*
			* @param Other  - 3x3 symetric matrix
			*
			* @return  ThisMatrix * Other
			*/
			SymmetricMatrix operator* (const SymmetricMatrix& Other) const 
			{
				SymmetricMatrix Result( Data[0] * Other[0] + Data[1] * Other[1] + Data[2] * Other[2],  Data[0] * Other[1] + Data[1] * Other[3] + Data[2] * Other[4],  Data[0] * Other[2] + Data[1] * Other[4] + Data[2] * Other[5],
					                                                                                   Data[1] * Other[1] + Data[3] * Other[3] + Data[4] * Other[4],  Data[1] * Other[2] + Data[3] * Other[4] + Data[4] * Other[5],
					                                                                                                                                                  Data[2] * Other[2] + Data[4] * Other[4] + Data[5] * Other[5] );
				return Result;
			}

			/**
			* Produce a new SymetricMatrix that is the result of SymetricMatrix times scalar
			* NB: ThisMatrix * Scalar
			*
			* @param Scalar  - single scale.
			*
			* @return  ThisMatrix * Scalar
			*/
			SymmetricMatrix operator*(const double Scalar) const
			{
				SymmetricMatrix Result(*this);
				Result *= Scalar;
				return Result;
			}


			/**
			* Update this Matrix to all zero values.
			*/
			void Zero()
			{
				Reset();
			}

			/**
			* Update this Matrix to an identity matrix (1 on diagonal 0 off diagonal).
			*/
			void Identity()
			{
				Data[0] = 1.;  Data[1] = 0.;  Data[2] = 0.;
				Data[3] = 1.;  Data[4] = 0.;
				Data[5] = 1.;
			}

			/**
			* Determinant of the matrix.   The matrix may be inverted if this is non-zero.
			*/
			SymmetricMatrix::ScalarType Det() const
			{

				return    -Data[2] * Data[2] * Data[3] +
					  2. * Data[1] * Data[2] * Data[4] +
				          -Data[0] * Data[4] * Data[4] +
					      -Data[1] * Data[1] * Data[5] +
					       Data[0] * Data[3] * Data[5];
			}

			

			/**
			* Construct the inverse of this matrix.
			* @param  Success   - On return this will be true if the inverse is valid
			* @param  Threshold - Compared against the determinant of the matrix.  If abs(Det()) < Threshold the inverse is said to have failed
			*
			* @return The inverse of 'this' matrix.
			*/
			SymmetricMatrix Inverse(bool& Success, double Threshold = 1.e-8) const
			{
				SymmetricMatrix Result( -Data[4] * Data[4] + Data[3] * Data[5],  Data[2] * Data[4] - Data[1] * Data[5], -Data[2] * Data[3] + Data[1] * Data[4],
					                                                            -Data[2] * Data[2] + Data[0] * Data[5],  Data[1] * Data[2] - Data[0] * Data[4],
					                                                                                                    -Data[1] * Data[1] + Data[0] * Data[3] );

				double InvDet = Det();
				Success = (FMath::Abs(InvDet) > Threshold);

				InvDet = 1. / InvDet;


				Result *= InvDet;

				return Result;
			}

			/**
			* Construct the inverse of this matrix.
			* NB: If abs( Det() ) of the matrix is less than 1.e-8 the result will be invalid.
			*
			* @return The inverse of 'this' matrix
			*/
			SymmetricMatrix Inverse() const
			{
				bool Success;
				return Inverse(Success);
			}

		private:
			static const int32 Mapping[9];
		};

		/**
		* Produce a new SymetricMatrix that is the result of SymetricMatrix times scalar
		* NB: ThisMatrix * Scalar
		*
		* @param Scalar  - single scale.
		*
		* @return  ThisMatrix * Scalar
		*/
		static SymmetricMatrix operator*(double Scalar, SymmetricMatrix Mat)
		{
			return Mat * Scalar;
		}

		/**
		* vector * Matrix.
		*  NB: This is really vector^Transpose * Matrix
		*      and results in a vector^Transpose.   But since we don't distinguish 
		*      the transpose space, we treat both as vectors.
		* @param LhsVector - vector
		* @param SymMatrix - SymmetricMatrix,
		*/
		static Vec3d operator*(const Vec3d& LhsVector, const SymmetricMatrix& SymMatrix)
		{
			// Vector^Transpose * Matrix = (Matrix^Transpose * Vector )^Transpose.
			// but our matrix is symmetric (Matrix^Transpose = Matrix)
			return SymMatrix * LhsVector;
		}


		// Double precision 3x3 matrix class
		class DMatrix : public ArrayBase<9, DMatrix>
		{
		public:
			typedef ArrayBase<9, DMatrix>            MyBase;
			typedef typename MyBase::ScalarType      ScalarType;

			DMatrix() 
			{
				Reset();
			}
			/**
			* Constructor: Element by element. 
			*/
			DMatrix(double a11, double a12, double a13,
				    double a21, double a22, double a23,
				    double a31, double a32, double a33) 
			{
				Data[0] = a11; Data[1] = a12; Data[2] = a13;
				Data[3] = a21; Data[4] = a22; Data[5] = a23;
				Data[6] = a31; Data[7] = a32; Data[8] = a33;
			}

			/**
			* Constructor: Row based. 
			*/
			DMatrix(const Vec3d& Row0,
				    const Vec3d& Row1,
				    const Vec3d& Row2)
			{
				Data[0] = Row0[0];  Data[1] = Row0[1];  Data[2] = Row0[2];
				Data[3] = Row1[0];  Data[4] = Row1[1];  Data[5] = Row1[2];
				Data[6] = Row2[0];  Data[7] = Row2[1];  Data[8] = Row2[2];
			}

			/**
			* Accesses elements in the matrix using standard M(i,j) notation

			* @param i - the row  in [0,3)
			* @param j - the column in [0, 3)
			*/
			DMatrix::ScalarType operator()(int32 i, int32 j) const
			{
				checkSlow(-1 < i && i < 3 && -1 < j && j < 3);
				return Data[j + i * 3];
			}

			DMatrix::ScalarType& operator()(int32 i, int32 j)
			{
				checkSlow(-1 < i && i < 3 && -1 < j && j < 3);
				return Data[j + i * 3];
			}

			/**
			* Produce a new Vec3d that is the result of Matrix vector multiplication.
			* NB: this does M*v  not v * M
			*
			* @param Vect  - three dimensional double precision vector
			*
			* @return  Matrix * Vect.
			*/
			Vec3d operator* (const Vec3d& Vect) const
			{
				Vec3d Result(
					Vect(0) * Data[0] + Vect(1) * Data[1] + Vect(2) * Data[2],
					Vect(0) * Data[3] + Vect(1) * Data[4] + Vect(2) * Data[5],
					Vect(0) * Data[6] + Vect(1) * Data[7] + Vect(2) * Data[8]
				);

				return Result;
			}

			/**
			* Produce a new 3x3 matrix that is the result of 3x3matrix  times 3x3matridx
			* NB: ThisMatrix * Other
			*
			* @param Other  - 3x3 full matrix
			*
			* @return  ThisMatrix * Other
			*/
			DMatrix operator*(const DMatrix& B) const
			{

				DMatrix Result(
					Data[0] * B[0] + Data[1] * B[3] + Data[2] * B[6],   Data[0] * B[1] + Data[1] * B[4] + Data[2] * B[7],  Data[0] * B[2] + Data[1] * B[5] + Data[2] * B[8] ,
					Data[3] * B[0] + Data[4] * B[3] + Data[5] * B[6],   Data[3] * B[1] + Data[4] * B[4] + Data[5] * B[7],  Data[3] * B[2] + Data[4] * B[5] + Data[5] * B[8] ,
					Data[6] * B[0] + Data[7] * B[3] + Data[8] * B[6],   Data[6] * B[1] + Data[7] * B[4] + Data[8] * B[7],  Data[6] * B[2] + Data[7] * B[5] + Data[8] * B[8] 
				);

				return Result;
			}

			/**
			* Update this Matrix to an identity matrix (1 on diagonal 0 off diagonal).
			*/
			void Identity()
			{
				Data[0] = 1.;  Data[1] = 0.;  Data[2] = 0.;
				Data[3] = 0.;  Data[4] = 1.;  Data[5] = 0.;
				Data[6] = 0.;  Data[7] = 0.;  Data[8] = 1.;
			}

			/**
			* Determinant of the matrix.   The matrix may be inverted if this is non-zero.
			*/
			DMatrix::ScalarType Det() const
			{

				return -Data[2] * Data[4] * Data[6] +
					    Data[1] * Data[5] * Data[6] +
					    Data[2] * Data[3] * Data[7] +
					   -Data[0] * Data[5] * Data[7] +
					   -Data[1] * Data[3] * Data[8] +
					    Data[0] * Data[4] * Data[8];

			}

			/**
			* Construct the inverse of this matrix.
			* @param  Success   - On return this will be true if the inverse is valid
			* @param  Threshold - Compared against the determinant of the matrix.  If abs(Det()) < Threshold the inverse is said to have failed
			*
			* @return The inverse of 'this' matrix.
			*/
			DMatrix Inverse(bool& sucess, double threshold = 1.e-8) const
			{
				DMatrix Result(
				   -Data[5] * Data[7] + Data[4] * Data[8],   Data[2] * Data[7] - Data[1] * Data[8],  -Data[2] * Data[4] + Data[1] * Data[5] ,
					Data[5] * Data[6] - Data[3] * Data[8],  -Data[2] * Data[6] + Data[0] * Data[8],   Data[2] * Data[3] - Data[0] * Data[5] ,
				   -Data[4] * Data[6] + Data[3] * Data[7],   Data[1] * Data[6] - Data[0] * Data[7],  -Data[1] * Data[3] + Data[0] * Data[4] 
				);

				double InvDet = Det();
				sucess = (FMath::Abs(InvDet) > threshold);

				InvDet = 1. / InvDet;

				Result *= InvDet;

				return Result;
			}

			/**
			* Construct the inverse of this matrix.
			* NB: If abs( Det() ) of the matrix is less than 1.e-8 the result will be invalid.
			*
			* @return The inverse of 'this' matrix
			*/
			DMatrix Inverse() const
			{
				bool tmp;
				return Inverse(tmp);
			}

			/**
			* Sum of the rows - return as a vector
			*/
			Vec3d RowSum() const
			{
				Vec3d Result( Data[0] + Data[1] + Data[2],
					          Data[3] + Data[4] + Data[5],
					          Data[6] + Data[7] + Data[8] );

				return Result;
			}

			/**
			* Sum of the column - return as a vector
			*/
			Vec3d ColSum() const 
			{
				Vec3d Result( Data[0] + Data[3] + Data[6],
					          Data[1] + Data[4] + Data[7],
					          Data[2] + Data[5] + Data[8] );
				return Result;
			}
		private:
	
		};

		/**
		 * left multiply matrix by vector.    This is really Transpose(vector) * Matrix
		 */
		static Vec3d operator*(const Vec3d& VecTranspose, const DMatrix& M)
		{
			Vec3d Result( M[0] * VecTranspose[0] + M[3] * VecTranspose[1] + M[6] * VecTranspose[2],
				          M[1] * VecTranspose[0] + M[4] * VecTranspose[1] + M[7] * VecTranspose[2],
				          M[2] * VecTranspose[0] + M[5] * VecTranspose[1] + M[8] * VecTranspose[2] );

			return Result;
		}

		/**
		* Dense3x3 matrix time SymmetricMatrix = results in dense3x3 matrix
		*/
		static DMatrix operator*(const DMatrix& Dm, const SymmetricMatrix& Sm)
		{
			DMatrix Result( Sm[0] * Dm[0] + Sm[1] * Dm[1] + Sm[2] * Dm[2],   Sm[1] * Dm[0] + Sm[3] * Dm[1] + Sm[4] * Dm[2],   Sm[2] * Dm[0] + Sm[4] * Dm[1] + Sm[5] * Dm[2] ,
				            Sm[0] * Dm[3] + Sm[1] * Dm[4] + Sm[2] * Dm[5],   Sm[1] * Dm[3] + Sm[3] * Dm[4] + Sm[4] * Dm[5],   Sm[2] * Dm[3] + Sm[4] * Dm[4] + Sm[5] * Dm[5] ,
				            Sm[0] * Dm[6] + Sm[1] * Dm[7] + Sm[2] * Dm[8],   Sm[1] * Dm[6] + Sm[3] * Dm[7] + Sm[4] * Dm[8],   Sm[2] * Dm[6] + Sm[4] * Dm[7] + Sm[5] * Dm[8] );


			return Result;
		}

		/**
		* SymmetricMatrix X Dense3x3 matrix  = results in dense3x3 matrix
		*/
		static DMatrix operator*(const SymmetricMatrix& Sm, const DMatrix& Dm)
		{
			DMatrix Result(  Sm[0] * Dm[0] + Sm[1] * Dm[3] + Sm[2] * Dm[6],   Sm[0] * Dm[1] + Sm[1] * Dm[4] + Sm[2] * Dm[7],   Sm[0] * Dm[2] + Sm[1] * Dm[5] + Sm[2] * Dm[8] ,
				             Sm[1] * Dm[0] + Sm[3] * Dm[3] + Sm[4] * Dm[6],   Sm[1] * Dm[1] + Sm[3] * Dm[4] + Sm[4] * Dm[7],   Sm[1] * Dm[2] + Sm[3] * Dm[5] + Sm[4] * Dm[8] ,
				             Sm[2] * Dm[0] + Sm[4] * Dm[3] + Sm[5] * Dm[6],   Sm[2] * Dm[1] + Sm[4] * Dm[4] + Sm[5] * Dm[7],   Sm[2] * Dm[2] + Sm[4] * Dm[5] + Sm[5] * Dm[8] );

			return Result;
		}

		/**
		* Double Precision Sparse Vector: - used with the SparseBMatrix in Quadric calculation
		*/
		class SparseVecD
		{
		public:

			typedef TMap< int32, double>  SparseContainer;


			SparseVecD()
			{}

			SparseVecD(const SparseVecD& Other)
			{
				SparseData = Other.SparseData;
			}

			/**
			* @return true if is the vector is empty
			*/
			bool bIsEmpty() const
			{
				return (0 == SparseData.Num());
			}

			/**
			* Empty this sparse vector
			*/
			void Reset()
			{
				SparseData = SparseContainer(); 
			}

			/**
			* Add element V[j] = Value
			*/
			void SetElement(const int32 j, double Value)
			{
				SparseData.Add(j, Value);
			}

			/**
			* Return Element V[j].  By default V[j] will be
			* zero if nothing has been stored there.
			*/
			double GetElement(const int32 j) const
			{
				const double* Result = SparseData.Find(j);
			
				return (Result != nullptr) ? *Result : 0.;
			}

			/**
			* Update the values of this vector to match Other
			*/
			SparseVecD& operator=(const SparseVecD& Other)
			{
				SparseData = Other.SparseData;
				return *this;
			}
		
			/**
			* Add another SparseVector to this one. 
			*/
			SparseVecD& operator+=(const SparseVecD& other)
			{
				for (auto citer = other.SparseData.CreateConstIterator(); citer ; ++citer)
				{
					AddToElement(citer->Key, citer->Value);
				}

				return *this;
			}

			/**
			* Scale this sparse vector with the scalar
			*/
			SparseVecD& operator*= (const double Scalar)
			{
				for (auto iter = SparseData.CreateIterator(); iter;  ++iter)
				{
					iter->Value *= Scalar;
				}

				return *this;
			}

			/**
			* Equality test against another SparseVector
			*/
			bool operator==(const SparseVecD& Other) const
			{

				return SparseData.OrderIndependentCompareEqual(Other.SparseData);

#if 0
				// This is a strong equality: it assumes both maps are sorted the same way.
				// this stinks because TMap doesn't have an internal sorting guarantee. 
				bool bEqual = (SparseData.Num() == Other.SparseData.Num());

				for (auto citer = SparseData.CreateConstIterator(), citerOther = Other.SparseData.CreateConstIterator(); bEqual && citer; ++citer, ++citerOther)
				{
					bEqual = bEqual && (citer->Key == citerOther->Key) && (citer->Value == citerOther->Value);
				}

				return bEqual;
#endif
			}

			bool operator!=(const SparseVecD& Other) const
			{
				return !operator==(Other);
			}

			/**
			* Compute Sum(  V[i] * Other[i] )
			*/
			double DotProduct(const SparseVecD& Other) const
			{
				double Result = 0.;
				for (auto citer = SparseData.CreateConstIterator(); citer; ++citer)
				{
					Result += (citer->Value) * Other.GetElement(citer->Key);
				}

				return Result;
			}

			/**
			* Compute the dot product of this sparse vector with itself : Sum(  V[i] * V[i] )
			*/
			double L2NormSqr() const
			{
				double Result = 0.;
				for (auto citer = SparseData.CreateConstIterator(); citer; ++citer)
				{
					const double Value = citer->Value;
					Result += Value * Value;
				}

				return Result;

			}

			/**
			* Sum the non-zero elements in a sparse array. Sum(  V[i]  )
			*/
			double SumValues() const
			{
				double Result = 0.;
				for (auto citer = SparseData.CreateConstIterator(); citer; ++citer)
				{
					const double Value = citer->Value;
					Result += Value;
				}

				return Result;
			}

			/**
			* Access to the underlying sparse data structure.
			*/
			SparseVecD::SparseContainer& GetData()             { return SparseData; }
			const SparseVecD::SparseContainer& GetData() const { return SparseData; }
			

			/**
			* @return the number of non-empty elements in the sparse array.
			*/
			int32 Num() const { return SparseData.Num(); }

		protected:

			void AddToElement(const int32 j, const double Value)
			{
				double* Data = SparseData.Find(j);
				if (Data != nullptr)
				{
					*Data += Value;
				}
				else
				{
					SparseData.Add(j, Value);
				}
			}

			SparseContainer SparseData;
		};

		/**
		* Special double precision vector wrapper.
		* NB:  This does not own the memory. It just
		* grants nicer semantics to some existing memory
		*/
		template <typename DataType>
		class TDenseArrayWrapper
		{
		public:
			typedef DataType    Type;

			TDenseArrayWrapper(DataType* data, int32 size)
			{
				Data = data;
				Size = size;
			}

			TDenseArrayWrapper(TArrayView<DataType> InArrayView)
			{
				Data = InArrayView.GetData();
				Size = InArrayView.Num();
			}

			int32 Num() const { return Size; }

			void SetElement(const int32 j, const DataType Value) { checkSlow(j < Size);  Data[j] = Value; }
			DataType GetElement(const int32 j) const { checkSlow(j < Size);  return Data[j]; }

			DataType&  operator[] (const int32 j) { checkSlow(j < Size);  return Data[j]; }
			DataType operator[] (const int32 j) const { checkSlow(j < Size); return Data[j]; }
			DataType& operator() (const int32 j) { return operator[](j); }
			DataType operator() (const int32 j) const { return operator[](j); }

			TDenseArrayWrapper& operator*= (double Scalar)
			{
				int32 Num = Size;
				for (int32 i = 0; i < Num; ++i) Data[i] *= Scalar;

				return  *this;
			}
			TDenseArrayWrapper& operator+= (const TDenseArrayWrapper& Other)
			{
				int32 Num = Size;
				checkSlow(Num == Other.Num());

				for (int32 i = 0; i < Num; ++i) Data[i] += Other.Data[i];

				return *this;
			}


			DataType DotProduct(const TDenseArrayWrapper& Other) const
			{
				int32 Num = FMath::Min(Size, Other.Num());

				double Result = 0.;
				for (int32 i = 0; i < Num; ++i)
				{
					Result += Data[i] * Other[i];
				}

				return Result;
			}

			double L2NormSqr() const
			{
				return DotProduct(*this);
			}

			bool operator==(const TDenseArrayWrapper& Other) const
			{
				int32 Num = Size;
				bool Result = (Num == Other.Size);

				for (int32 i = 0; i < Num && Result; ++i)
				{
					Result = Result && (Data[i] == Other.Data[i]);
				}

				return Result;
			}

			bool operator!=(const TDenseArrayWrapper& Other) const
			{
				return !operator==(Other);
			}

		private:

			DataType * Data; // Note: this object doesn't own the data!
			int32      Size;
		};


		/**
		* Fixed length double precision vector class
		* with dot product and get/set element methods
		* that are consistent with the sparse double precision vector class SparseVecD
		*/
		template <int32 SIZE>
		class TDenseVecD : public ArrayBase<SIZE, TDenseVecD<SIZE>>
		{
		public:

			typedef ArrayBase<SIZE, TDenseVecD>           MyBase;
			typedef typename MyBase::ScalarType           ScalarType;

			using MyBase::Data;
			using MyBase::Reset;

			enum { Size = SIZE };

			TDenseVecD()
			{
				Reset();
			}

			/**
			* Construct from Wrapped  Array.
			*/
			TDenseVecD(const TDenseArrayWrapper<float>& FloatArrayWrapper)
			{
				int32 NumElements = FloatArrayWrapper.Num();
				checkSlow(NumElements == SIZE);

				for (int32 i = 0; i < SIZE; ++i)
				{
					Data[i] = FloatArrayWrapper[i];
				}
			}

			TDenseVecD(const TDenseVecD& Other) = default;
			TDenseVecD& operator=(const TDenseVecD& Other) = default;

			void SetElement(const int32 j, const double Value) { MyBase::operator[](j) = Value; }
			ScalarType GetElement(const int32 j) const { return MyBase::operator[](j); }

			/**
			* Sum of element-by-element product.
			* Dot Product = Sum( this[i] * Other[i] )
			*
			* @return sum of element-by-element multiplicaiton.
			*/
			double DotProduct(const TDenseVecD& Other) const
			{
				double Result = 0.;
				for (int32 i = 0; i < SIZE; ++i)
				{
					Result += Data[i] * Other[i];
				}

				return Result;
			}

		};

		/**
		* Sparse Matrix class with 3 rows and an unknown number of columns.
		* 3 x M matrix.
		*
		* The sparsity is encoded as the existence / non-existence of a column.
		* a non-filled column is assumed to be all zeros.
		*
		*/
		class SparseBMatrix
		{
		public:
		
			typedef TMap< int32, Vec3d >   SparseContainer;

			// empty the matrix
			void Reset()
			{
				SparseData = SparseContainer();
				//SparseData.Empty();
			}

			/**
			* Add a new, or over-write and existing column.
			* @param j          - column index.
			* @param ColumnVec  - the new values for the column.
			*/
			void SetColumn(const int32 j, const Vec3d& ColumnVec)
			{
				SparseData.Add(j, ColumnVec);
			}

			/**
			* Copy of data stored in the column. 
			* @param j          - column index.
			*/
			Vec3d GetColumn(const int32 j) const
			{
				const Vec3d* Result = SparseData.Find(j);
				
				return (Result != nullptr) ? *Result : Vec3d(0., 0., 0.);
			}

			/**
			* Look up data in matrix with Row, Column address. 
			*/
			double operator()(int32 i, int32 j) const
			{
				return GetColumn(j)[i];
			}

			/**
			* Add another SparseBMatrix to this one
			* Element by element addition.
			* @param Other - the matrix to be added.  
			*/
			SparseBMatrix& operator+=(const SparseBMatrix& Other)
			{
				for (auto Citer = Other.SparseData.CreateConstIterator(); Citer; ++Citer)
				{
					AddToColumn(Citer->Key, Citer->Value);
				}

				return *this;
			}

			/**
			* Matrix multiplication with a sparse vector.
			* SparseMatrix * SparseVector  [3 x m] . [m] = 3 vector. 
			* @param SparseVec - vector
			*/
			Vec3d operator*(const SparseVecD& SparseVec) const
			{
				Vec3d Result; // default 0 initialization
				for (auto Citer = SparseData.CreateConstIterator(); Citer; ++Citer)
				{
					double Scalar = SparseVec.GetElement(Citer->Key);
					Result += Scalar * (Citer->Value);
				}

				return Result;
			}

			/**
			* Rescale this sparse matrix by *= each element.
			* @param Scalar - Value to scale each element with.
			*/
			SparseBMatrix& operator*= (const double Scalar)
			{
				for (auto Citer = SparseData.CreateIterator(); Citer; ++Citer)
				{
					Citer->Value *= Scalar;
				}
				return *this;
			}

			/**
			* Const access to the underlying sparse data structure.
			*/ 
			const SparseBMatrix::SparseContainer& GetData() const
			{
				return SparseData;
			}

			
			friend SymmetricMatrix OuterProductOperator(const SparseBMatrix& matrix);
			friend SparseVecD operator*(const Vec3d VecTranpose, const SparseBMatrix& SparseB);

		private:

			void AddToColumn(const int32 j, const Vec3d& ColumnVec)
			{
				Vec3d* Data = SparseData.Find(j);
				if (Data != nullptr)
				{
					*Data += ColumnVec;
				}
				else
				{
					SparseData.Add(j, ColumnVec);
				}
			}

			SparseContainer  SparseData;

		};


		/**
		* A Dense alternative to the SparseBMatrix
		* NUMCOLS is the number of columns
		*
		* This double precision matrix has 3 rows and NUMCOLS columns.
		* 
		*/
		template<int32 NUMCOLS>
		class TDenseBMatrix
		{
		public:

			typedef Vec3d  MyStorageType[NUMCOLS];


			TDenseBMatrix()
			{
				// zero the data
				Reset();
			}

			TDenseBMatrix(const TDenseBMatrix& Other)
			{
				memcpy(Data, Other.Data, sizeof(Data));
			}

			/**
			* Zero all enteries.
			*/
			void Reset()
			{
				memset(Data, 0, sizeof(Data));
			}

			/**
			* Replace data in the j-th column with the given column vector
			* @param j - column index
			* @param ColumnVec - the column to insert.
			*/
			void SetColumn(const int32 j, const Vec3d& ColumnVec)
			{
				checkSlow(j < NUMCOLS);  Data[j] = ColumnVec;
			}

			/**
			* Access to the j-th column.
			*/
			Vec3d& GetColumn(const int32 j)
			{
				checkSlow(j < NUMCOLS);
				return Data[j];
			}

			const Vec3d& GetColumn(const int32 j) const
			{
				checkSlow(j < NUMCOLS);
				return Data[j];
			}

			/**
			* Row-Column access to data in the matrix
			*/
			double operator()(const int32 i, const int32 j) const
			{
				return GetColumn(j)(i);
			}
			/**
			* Row-Column access to data in the matrix
			*/
			double& operator()(const int32 i, const int32 j)
			{
				return GetColumn(j)(i);
			}		

			/**
			* Number of columns that define this matrix.  Known at compile time.
			*/
			int32 NumCols() const { return NUMCOLS; }

			/**
			* Assignment 
			*/
			TDenseBMatrix& operator=(const TDenseBMatrix& Other)
			{
				memcpy(Data, Other.Data, sizeof(Data));
				return *this;
			}

			/**
			* Element-by-element addition with another matrix.
			*/
			TDenseBMatrix& operator+= (const TDenseBMatrix& Other)
			{
				for (int32 i = 0; i < NUMCOLS; ++i) Data[i] += Other.Data[i];
				return *this;
			}

			/**
			* Matrix multiplication with a correctly sized dense vector.
			* DenseBMatrix * DenseVector  [3 x m] . [m] = 3 vector.
			*/
			Vec3d operator*(const TDenseVecD<NUMCOLS>& DenseVec) const
			{
				Vec3d Result(0., 0., 0.); // default zero initialization
				for (int32 i = 0; i < NUMCOLS; ++i) Result += DenseVec[i] * Data[i];

				return Result;
			}

			/**
			* Rescale this 3xNUMCOLS matrix by *= each element.
			* @param Scalar - Value to scale each element with.
			*/
			TDenseBMatrix& operator*= (const double Scalar)
			{
				for (int32 i = 0; i < NUMCOLS; ++i) Data[i] *= Scalar;
				return *this;
			}

		private:

			// Array of column vectors defines the matrix
			MyStorageType Data;

		};

		/**
		* Transpose(Vector) * Matrix.
		*/
		template <int32 SIZE>
		TDenseVecD<SIZE> operator*(const Vec3d VecTranspose, const TDenseBMatrix<SIZE>& DenseB)
		{
			TDenseVecD<SIZE> Result;

			for (int32 i = 0; i < SIZE; ++i)
			{
				Result(i) = DenseB.GetColumn(i).DotProduct(VecTranspose);
			}

			return Result;
		}


		/**
		* Transpose(Vector) * Matrix.
		*/
		inline SparseVecD operator* (const Vec3d VecTranpose, const SparseBMatrix& SparseB)
		{
			SparseVecD Result;
			for (auto iter = SparseB.SparseData.CreateConstIterator(); iter; ++iter)
			{
				const auto VecValue = iter->Value;
				double DotValue = VecValue.DotProduct(VecTranpose);
				Result.SetElement(iter->Key, DotValue);
			}

			return Result;
		}

		/**
		* Construct the outer product Va . Transpose(Va)
		* This will result in a operator that will give the scaled projection of a vector
		* onto Va.  (scaled by va.lenghtsqrd)
		*/
		static SymmetricMatrix ScaledProjectionOperator(const Vec3d& Vect)
		{
			SymmetricMatrix Result( Vect[0] * Vect[0],  Vect[0] * Vect[1],  Vect[0] * Vect[2],
				                                        Vect[1] * Vect[1],  Vect[1] * Vect[2],
			                                                                Vect[2] * Vect[2] );

			return Result;
		}

		/**
		* Returns B. Transpose(B)
		*/
		template<int32 SIZE>
		SymmetricMatrix OuterProductOperator(const TDenseBMatrix<SIZE>& DenseB)
		{
			// counting on return value optimization
			SymmetricMatrix Result;  // default zero initialization

			for (int32 i = 0; i < SIZE; ++i)  Result += ScaledProjectionOperator(DenseB.GetColumn(i));
		
			return Result;
		}


		/**
		* Returns B. Transpose(B)
		*/
		inline SymmetricMatrix OuterProductOperator(const SparseBMatrix& SparseB)
		{
			// counting on return value optimization
			SymmetricMatrix Result; // default zero initialization

			for (auto citer = SparseB.SparseData.CreateConstIterator(); citer ; ++citer)
			{
				Result += ScaledProjectionOperator(citer->Value);
			}

			return Result;
		} 

		/**
		* This class generates the interpolation coefficients vector (Vec3d) g  and distance (double) d
		* defined over the face of a triangle.  It does this in one of two ways.  The first constructs a local basis relative to the 
		* triangle.  This is preferable.
		* 
		* The second is the so-called "legacy method" it uses position vectors for the vertices and will fall back to a crude approximation
		* when the triangle lies on a plane that intersects the origin.
		*
		* *
		* The actual system solved is:
		*  <pa | g> + d = s0
		*  <pb | g> + d = s1
		*  <pc | g> + d = s2
		*  <FaceNormal | g> = 0
		*
		* But we re-write this as
		* <(pc - pa) | g >  = s2 - s0
		* <(pb - pa) | g >  = s1 - s0
		* < FaceNormal | g > = 0
		* d = s0 - <pa | g>
		*
		* The first three equations can be solved for "g", in terms of s, and the result can be used to
		* compute d.
		*
		* The BasisMatrix is defined in terms of the three corners of triangle
		*  {pa, pb, pc} and the normal corresponding normal 'FaceNormal'
		*
		* BasisMatrix =
		*   ( pc_0 - pa_0 , pc_1 - pa_1 , pc_2 - pa_2 )
		*   ( pb_0 - pa_0 , pb_1 - pa_1 , pb_2 - pa_2)
		*   ( FaceNormal_0, FaceNormal_1, FaceNormal_2 )
		* The rows corresponds to vectors along two sides of the triangle and the face normal
		*
		* Solving for "g" is simply solving
		*  BasisMatrix * g = vec(s2-s0, s1-s0, 0)
		*
		* or
		*  g  = Invserse(BasisMatrix) * vec(s2-s0, s1-s0, 0)
		* the Inverse(BasisMatrix) is cached as the CoBasisMatrix
		*
		*
		* The computation is broken up do allow for reuse with multiple sets of per-vertex data.
		* 
		* ------------------------------------------
		* Alternately when using the legacy method
		* ------------------------------------------
		* The Position Matrix is defined in terms of the three corners of triangle
		*  {pa, pb, pc} with corresponding normal 'FaceNormal'
		*
		* Position Matrix =
		*   ( pa_0, pa_1, pa_2 )
		*   ( pb_0, pb_1, pb_2 )
		*   ( pc_0, pc_1, pc_2 )
		*
		* The actual system solved is:
		*  <pa | g> + d = s0
		*  <pb | g> + d = s1
		*  <pc | g> + d = s2
		*  <FaceNormal | g> = 0
		*
		*
		* In matrix form   ( PositionMatrix   Vec3(1) )   ( g )  = ( s )
		*                  ( FaceNormal^T ,     0     )   ( d )    ( 0 )
		*
		* where the vector (Vec3d) 's' represents the per-vertex data that forms boundary conditions
		* for the interpolation.
		*
		*  The actual solution is given by:
		*  -- Distance
		*  double d = < FaceNormal | InvsPositionMatrix * s> / <FaceNormal | InversePositionMatrix * Vec3d(1) >;
		*  -- Gradient
		*  Vec3d  g = InversePositionMatrix * s - d * InversePositionMatrix * Vec3d(1);
		*
		* The computation is broken up do allow for reuse with multiple sets of per-vertex data.
		*/
		class InverseGradientProjection
		{
		public:
			InverseGradientProjection(const Vec3d& Pos0, const Vec3d& Pos1, const Vec3d& Pos2, const Vec3d& FaceNormal, bool bUseLegacyGradientMethod)
			: bLegacyGradientMethod(bUseLegacyGradientMethod)
			{
				// Threshold for computing the matrix inverse.
				constexpr  double DetThreshold = 1.e-8;

				if (bLegacyGradientMethod)
				{
					// re-use matrix and vector for the legacy version
					DMatrix& PosInv = CoBasisMatrix;
					Vec3d& Dhat = Origin; // n * Inverse(PositinoMatrix) / < n | Inverse(PositionMatrix) Vec3(1) >

					// row initialization 
					DMatrix PositionMatrix(Pos0, Pos1, Pos2);

					// Compute the inverse of the position matrix
					PosInv = PositionMatrix.Inverse(bIsValid, DetThreshold);
					if (bIsValid)
					{
						// InversePositionMatrix * Vec3(1)
						MInv1 = PosInv.RowSum();

						// <FaceNormal | InversePositionMatrix 
						Dhat = FaceNormal * PosInv;

						// <FaceNormal | InvesePositionMatrix Vec3d(1)> 
						double ReScale = Dhat[0] + Dhat[1] + Dhat[2];

						bIsValid = bIsValid && (FMath::Abs(ReScale) > 1.e-8);

						// divide by <FaceNormal | InvesePositionMatrix Vec3d(1)> 
						Dhat *= (1. / ReScale);
						//now:  Dhat =  <FaceNormal | InvPos  / <FaceNormal | InvPos. Vec3d(1) >
					}

				}
				else // the standard method
				{ 
					Origin = Pos0;

					const Vec3d Side20 = Pos2 - Pos0;
					const Vec3d Side10 = Pos1 - Pos0;

					DMatrix BasisMatrix(Side20,
										Side10,
										FaceNormal);

					CoBasisMatrix = BasisMatrix.Inverse(bIsValid, DetThreshold);
				}
			}


			bool IsValid() const
			{
				return bIsValid;
			}

			// @param  PerVertexData - Vertex Data at vertex {0, 1, 2} stored as a Vec3d
			// @param  OutGradient   - on return, the gradient terms
			// @return  Distance
			double ComputeGradient(const Vec3d& PerVertexData, Vec3d& OutGradient) const
			{
				if (bLegacyGradientMethod)
				{
					// re-use matrix and vector for the legacy version
					const DMatrix& PosInv = CoBasisMatrix;
					const Vec3d& Dhat = Origin;

					// PosInv . s
					Vec3d MInvS = PosInv * PerVertexData;

					// Dist = <dhat | s>
					double Distance = Dhat.DotProduct(PerVertexData);

					// Grad =  PosInv . s - Dist * PosInv . (1, 1, 1} 
					OutGradient = MInvS - Distance * MInv1;

					return Distance;
				}
				else // the standard method
				{ 
					Vec3d DataVec(PerVertexData[2] - PerVertexData[0], PerVertexData[1] - PerVertexData[0], 0.);

					OutGradient = CoBasisMatrix * DataVec;
					double Distance = PerVertexData[0] - Origin.DotProduct(OutGradient);

					return Distance;
				}
			}

		private:

			bool    bIsValid;  // Indicates if the inversions incurred a divide by very-small-number.
		
			bool    bLegacyGradientMethod = false;

			// used by the standard method
			DMatrix CoBasisMatrix;   // Inverse( {P2-P0}, {P1 - P0}, Normal}
			Vec3d   Origin;          // corresponds to P0

			// used by the legacy version only
			Vec3d   MInv1;                   // Inverse(PositionMatrix) * Vec3(1)
		};

		template <typename SparseVecDOrDenseVecD>
		struct TVectorToMatrixTrait
		{
			typedef SparseBMatrix   BMatrixType;
		};

		template <>
		struct TVectorToMatrixTrait<SparseVecD>
		{
			typedef SparseBMatrix   BMatrixType;
		};

		template<>
		struct TVectorToMatrixTrait<TDenseVecD<3>>
		{
			typedef TDenseBMatrix<3>   BMatrixType;
		};



		
		/**
		*  Create a mask that holds the union of the sparse topology.
		*  The resulting array will 
		*
		*  @return  Array that holds the sparse topology.
		*/
		static TArray<int32> GetIterationMask(const SparseVecD& Attr0, const SparseVecD& Attr1, const SparseVecD& Attr2)
		{

			int32 MaxElement;
			{
				MaxElement = -1;
				for (auto citer = Attr0.GetData().CreateConstIterator(); citer; ++citer)
				{
					MaxElement = FMath::Max(MaxElement, citer->Key);
				}
				for (auto citer = Attr1.GetData().CreateConstIterator(); citer; ++citer)
				{
					MaxElement = FMath::Max(MaxElement, citer->Key);
				}
				for (auto citer = Attr2.GetData().CreateConstIterator(); citer; ++citer)
				{
					MaxElement = FMath::Max(MaxElement, citer->Key);
				}

			}


			TArray<int32> Result;

			if (MaxElement > -1)
			{
				//Result.resize(MaxElement + 1, 0);

				Result.AddZeroed(MaxElement + 1);

				// merge the sparsity  
				for (auto citer = Attr0.GetData().CreateConstIterator(); citer; ++citer)
				{
					Result[citer->Key] = 1;
				}
				for (auto citer = Attr1.GetData().CreateConstIterator(); citer; ++citer)
				{
					Result[citer->Key] = 1;
				}
				for (auto citer = Attr2.GetData().CreateConstIterator(); citer; ++citer)
				{
					Result[citer->Key] = 1;
				}
			}

			return Result;
		}

		template <int32 SIZE>
		struct DenseIterMask
		{
			int32 operator[](int32 i) const { return 1; }

			int32 Num() const { return SIZE; }
		};

		template <int32 SIZE>
		DenseIterMask<SIZE> GetIterationMask(const TDenseVecD<SIZE>& Attr0, const TDenseVecD<SIZE>& Attr1, const TDenseVecD<SIZE>& Attr2)
		{
			return DenseIterMask<SIZE>();
		}



		/**
		Axis aligned 2d bounding box class used for tracking and clamping UVs.
		*/
		class FAABBox2d
		{
		public:
			typedef float MyStorageType[4];

			/* default initialized puts the bbox in an invalid state*/
			FAABBox2d()
			{
				Reset();
			}

			FAABBox2d(const FAABBox2d& Other)
			{
				MinMax[0] = Other.MinMax[0];
				MinMax[1] = Other.MinMax[1];
				MinMax[2] = Other.MinMax[2];
				MinMax[3] = Other.MinMax[3];
			}

			/* Set to a default empty state */
			void Reset()
			{
				MinMax[0] = FLT_MAX;
				MinMax[1] = FLT_MAX;

				MinMax[2] = -FLT_MAX;
				MinMax[3] = -FLT_MAX;
			}
			/**
			* Expand this BBox to include the Other
			* @param Other - another Axis Aligned bbox
			*/
			void Union(const FAABBox2d& Other)
			{
				MinMax[0] = FMath::Min(MinMax[0], Other.MinMax[0]);
				MinMax[1] = FMath::Min(MinMax[1], Other.MinMax[1]);

				MinMax[2] = FMath::Max(MinMax[2], Other.MinMax[2]);
				MinMax[3] = FMath::Max(MinMax[3], Other.MinMax[3]);
			}

			/**
			* The += operator is defined to be a union of bboxes.
			*/
			FAABBox2d& operator+=(const FAABBox2d& Other)
			{
				Union(Other);
				return *this;
			}

			FAABBox2d& operator=(const FAABBox2d& Other)
			{
				MinMax[0] = Other.MinMax[0];
				MinMax[1] = Other.MinMax[1];
				MinMax[2] = Other.MinMax[2];
				MinMax[3] = Other.MinMax[3];
				return *this;
			}

			/**
			* @return true only if the min is not greater than the max
			*/
			bool IsValid() const
			{
				bool bIsValid = !(MinMax[0] > MinMax[2]) && !(MinMax[1] > MinMax[3]);
				return bIsValid;
			}

			/** Expand this bbox to include the provied point
			* @param Point - point to be included in this Axis aligned bounding box
			*/
			void ExpandToInclude(const FVector2f& Point)
			{
				MinMax[0] = FMath::Min(MinMax[0], Point.X);
				MinMax[1] = FMath::Min(MinMax[1], Point.Y);

				MinMax[2] = FMath::Max(MinMax[2], Point.X);
				MinMax[3] = FMath::Max(MinMax[3], Point.Y);
			}


			/**
			* Clamp values that exceed the bbox
			* @Param Point - point to be clamped by this bbox
			*/
			void ClampPoint(FVector2f& Point) const
			{
				Point.X = FMath::Clamp(Point.X, MinMax[0], MinMax[2]);
				Point.Y = FMath::Clamp(Point.Y, MinMax[1], MinMax[3]);
			}

			/**
			* Clamp values that exceed the bbox
			* @Param Point - point to be clamped by a padded version of this bbox
			* @Param Fraction - fraction of box width to use for padding.
			*/
			void ClampPoint(FVector2f& Point, const float Fraction) const
			{
				const float HalfFrac = Fraction * 0.5f;
				const float XPad = HalfFrac * (MinMax[2] - MinMax[0]);
				const float YPad = HalfFrac * (MinMax[3] - MinMax[1]);

				Point.X = FMath::Clamp(Point.X, MinMax[0] - XPad, MinMax[2] + XPad);
				Point.Y = FMath::Clamp(Point.Y, MinMax[1] - YPad, MinMax[3] + YPad);
			}

			/**
			* Min corner the bbox
			*/
			FVector2f Min() const
			{
				FVector2f BBoxMin = { MinMax[0], MinMax[1] };
				return BBoxMin;
			}

			/**
			* Max corner the bbox
			*/
			FVector2f Max() const
			{
				FVector2f BBoxMax = { MinMax[2], MinMax[3] };
				return BBoxMax;
			}
		private:

			MyStorageType MinMax;

		};

	}
}
