// Copyright Epic Games, Inc. All Rights Reserved.

#include "SkeletalSimplifierMeshManager.h"



/**
*  Slightly modified version of the quadric simplifier found in MeshSimplifier\Private\MeshSimplify.h
*  that code caries the copyright--
*/
// Copyright (C) 2009 Nine Realms, Inc

#define INVALID_EDGE_ID UINT32_MAX
 
SkeletalSimplifier::FSimplifierMeshManager::FSimplifierMeshManager(const MeshVertType* InSrcVerts, const uint32 InNumSrcVerts,
	const uint32* InSrcIndexes, const uint32 InNumSrcIndexes, const bool bMergeBonesIfSamePos)
	:
	bWeldBonesIfSamePos(bMergeBonesIfSamePos),
	NumSrcVerts(InNumSrcVerts),
	NumSrcTris(InNumSrcIndexes / 3),
	ReducedNumVerts(InNumSrcVerts),
	ReducedNumTris(InNumSrcIndexes / 3),
	EdgeVertIdHashMap(1 << FMath::Min(16u, FMath::FloorLog2(InNumSrcVerts)))
{
	// Allocate verts and tris
	VertArray.Empty();
	VertArray.AddDefaulted(NumSrcVerts);

	TriArray.Empty();
	TriArray.AddDefaulted(NumSrcTris);

	// Deep copy the verts

	for (uint32 i = 0; i < InNumSrcVerts; ++i)
	{
		VertArray[ i ].vert = InSrcVerts[ i ];
	}

	// Register the verts with the tris

	for (uint32 i = 0; i < (uint32)NumSrcTris; ++i)
	{
		uint32 Offset = 3 * i;
		for (uint32 j = 0; j < 3; ++j)
		{
			uint32 IndexIdx = Offset + j;
			uint32 VertIdx = InSrcIndexes[ IndexIdx ];

			checkSlow(IndexIdx < InNumSrcIndexes);
			checkSlow(VertIdx < InNumSrcVerts);

			TriArray[ i ].verts[ j ] = &VertArray[ VertIdx ];
		}
	}


	// Register each tri with the vert.

	for (int32 i = 0; i < NumSrcTris; ++i)
	{
		for (int32 j = 0; j < 3; ++j)
		{
			SimpTriType* TriPtr = &TriArray[ i ];
			SimpVertType* VertPtr = TriPtr->verts[j];

			VertPtr->adjTris.Add(TriPtr);
		}
	}


	// Group the verts that share the same location.

	GroupVerts(VertArray);

	// Set the Attribute ElementIDs on the verts - used to track split and non-split attributes. 
	// Note: this requires that GroupVerts() has already been called.

	SetAttributeIDS(VertArray);

	// Populate EdgeArray

	MakeEdges(VertArray, NumSrcTris, EdgeArray);

	// Links all the edges together
	GroupEdges(EdgeArray);

	{
		EdgeVertIdHashMap.Resize(EdgeArray.Num());
		{
			TArray<int32> HashValues;
			ResizeArray(HashValues, EdgeArray.Num());

			const auto* Edges = EdgeArray.GetData();

			for (int32 i = 0, I = EdgeArray.Num(); i < I; ++i)
			{
				HashValues[i] = HashEdge(Edges[i].v0, Edges[i].v1);
			}
			

			for (int32 i = 0, I = EdgeArray.Num(); i < I; i++)
			{
				EdgeVertIdHashMap.Add(HashValues[i], i);
			}
		}
	}

}

void SkeletalSimplifier::FSimplifierMeshManager::GroupVerts(TArray<SimpVertType>& Verts)
{
	const int32 NumVerts = Verts.Num();

	// group verts that share a point
	FHashTable HashTable(1 << FMath::Min(16u, FMath::FloorLog2(NumVerts / 2)), NumVerts);

	TArray<uint32> HashValues;
	ResizeArray(HashValues, NumVerts);
	{
		// Compute the hash values

		for (int32 i = 0; i < NumVerts; ++i)
		{
			HashValues[i] = HashPoint((FVector)Verts[i].GetPos());
		}

		// insert the hash values.
		for (int i = 0; i < NumVerts; i++)
		{
			HashTable.Add(HashValues[i], i);
		}
	}
	const bool bTestCoincidentBones = !bWeldBonesIfSamePos;

	for (int i = 0; i < NumVerts; i++)
	{
		SimpVertType* v1 = &Verts[i];

		// already grouped
		if (v1->next != v1)
		{
			continue;
		}

		// find any matching verts
		const uint32 hash = HashValues[i];


		for (int j = HashTable.First(hash); HashTable.IsValid(j); j = HashTable.Next(j))
		{

			SimpVertType* v2 = &Verts[j];

			if (v1 == v2)
				continue;

			// link if the verts have the same pos (and optionally same bones)
			if (IsCoincident(v1, v2, bTestCoincidentBones))
			{
				checkSlow(v2->next == v2);
				checkSlow(v2->prev == v2);

				// insert v2 after v1
				v2->next = v1->next;
				v2->next->prev = v2;

				v2->prev = v1;
				v1->next = v2;


			}
		}
	}
}

void SkeletalSimplifier::FSimplifierMeshManager::SetAttributeIDS(TArray<SimpVertType>& Verts)
{

	const int32 NumVerts = Verts.Num();

	// set element ids on the verts.

	VertPtrArray CoincidentSimpVerts;
	int NormalID = 0;
	int TangentID = 0;
	int BiTangetID = 0;
	int ColorID = 0;
	int UVIDs[MAX_TEXCOORDS];
	for (int i = 0; i < MAX_TEXCOORDS; ++i) UVIDs[i] = 0;

	for (int i = 0; i < NumVerts; ++i)
	{
		CoincidentSimpVerts.Reset();
		SimpVertType& HeadSimpVert = Verts[i];
		auto& HeadVertAttrs = HeadSimpVert.vert.BasicAttributes;

		// already processed this vert ( and all the coincident verts with it).
		if (HeadVertAttrs.ElementIDs.ColorID != -1) continue;

		// give this vert the next available element Id
		HeadVertAttrs.ElementIDs.NormalID = NormalID++;
		HeadVertAttrs.ElementIDs.TangentID = TangentID++;
		HeadVertAttrs.ElementIDs.BiTangentID = BiTangetID++;
		HeadVertAttrs.ElementIDs.ColorID = ColorID++;

		for (int t = 0; t < MAX_TEXCOORDS; ++t)
		{
			HeadVertAttrs.ElementIDs.TexCoordsID[t] = UVIDs[t]++;
		}

		// collect the verts in the linklist - they share the same location.
		GetVertsInGroup(HeadSimpVert, CoincidentSimpVerts);

		// The HeadSimpVert is the j=0 one..
		for (int j = 1; j < CoincidentSimpVerts.Num(); ++j)
		{
			SimpVertType* ActiveSimpVert = CoincidentSimpVerts[j];
			auto& ActiveVertAttrs = ActiveSimpVert->vert.BasicAttributes;
			checkSlow(ActiveVertAttrs.ElementIDs.NormalID == -1);
			
			// If normals match another vert at this location, they should share ID
			for (int k = 0; k < j; ++k)
			{
				SimpVertType* ProcessedSimpVert = CoincidentSimpVerts[k];
				auto& ProcessedVertAttrs = ProcessedSimpVert->vert.BasicAttributes;

				if (ProcessedVertAttrs.Normal == ActiveVertAttrs.Normal)
				{
					ActiveVertAttrs.ElementIDs.NormalID = ProcessedVertAttrs.ElementIDs.NormalID;
					break;
				}
			}
			// If normal didn't mach with any earlier vert at this location. Give new ID
			if (ActiveVertAttrs.ElementIDs.NormalID == -1)
			{
				ActiveVertAttrs.ElementIDs.NormalID = NormalID++;
			}

			// If tangents match another vert at this location, they should share ID
			for (int k = 0; k < j; ++k)
			{
				SimpVertType* ProcessedSimpVert = CoincidentSimpVerts[k];
				auto& ProcessedVertAttrs = ProcessedSimpVert->vert.BasicAttributes;

				if (ProcessedVertAttrs.Tangent == ActiveVertAttrs.Tangent)
				{
					ActiveVertAttrs.ElementIDs.TangentID = ProcessedVertAttrs.ElementIDs.TangentID;
					break;
				}

			}
			// If tangent didn't mach with any earlier vert at this location. Give new ID
			if (ActiveVertAttrs.ElementIDs.TangentID == -1)
			{
				ActiveVertAttrs.ElementIDs.TangentID = TangentID++;
			}

			// If Bitangents match another vert at this location, they should share ID
			for (int k = 0; k < j; ++k)
			{
				SimpVertType* ProcessedSimpVert = CoincidentSimpVerts[k];
				auto& ProcessedVertAttrs = ProcessedSimpVert->vert.BasicAttributes;

				if (ProcessedVertAttrs.BiTangent == ActiveVertAttrs.BiTangent)
				{
					ActiveVertAttrs.ElementIDs.BiTangentID = ProcessedVertAttrs.ElementIDs.BiTangentID;
					break;
				}
			}
			// If BiTangent didn't mach with any earlier vert at this location. Give new ID
			if (ActiveVertAttrs.ElementIDs.BiTangentID == -1)
			{
				ActiveVertAttrs.ElementIDs.BiTangentID = BiTangetID++;
			}

			// If Color match another vert at this location, they should share ID
			for (int k = 0; k < j; ++k)
			{
				SimpVertType* ProcessedSimpVert = CoincidentSimpVerts[k];
				auto& ProcessedVertAttrs = ProcessedSimpVert->vert.BasicAttributes;

				if (ProcessedVertAttrs.Color == ActiveVertAttrs.Color)
				{
					ActiveVertAttrs.ElementIDs.ColorID = ProcessedVertAttrs.ElementIDs.ColorID;
					break;
				}
			}
			//If  Color didn't mach with any earlier vert at this location. Give new ID
			if (ActiveVertAttrs.ElementIDs.ColorID == -1)
			{
				ActiveVertAttrs.ElementIDs.ColorID = ColorID++;
			}

			for (int t = 0; t < MAX_TEXCOORDS; ++t)
			{
				// look for texture match
				for (int k = 0; k < j; ++k)
				{
					SimpVertType* ProcessedSimpVert = CoincidentSimpVerts[k];
					auto& ProcessedVertAttrs = ProcessedSimpVert->vert.BasicAttributes;

					if (ProcessedVertAttrs.TexCoords[t] == ActiveVertAttrs.TexCoords[t])
					{
						ActiveVertAttrs.ElementIDs.TexCoordsID[t] = ProcessedVertAttrs.ElementIDs.TexCoordsID[t];

						break;
					}
				}
				// If TexCoord didn't mach with any earlier vert at this location. Give new ID
				if (ActiveVertAttrs.ElementIDs.TexCoordsID[t] == -1)
				{
					ActiveVertAttrs.ElementIDs.TexCoordsID[t] = UVIDs[t]++;
				}
			}

		}
	}

	// @todo Maybe add one pass at to split element bow-ties.  
	// These could have formed along a UV seam if a both sides of the seam shared the same value at a isolated vertex - but in that case a split vertex might not have been generated..

}

void SkeletalSimplifier::FSimplifierMeshManager::MakeEdges(const TArray<SimpVertType>& Verts, const int32 NumTris, TArray<SimpEdgeType>& Edges)
{

	// Populate the TArray of edges.
	const int32 NumVerts = Verts.Num();

	int32 maxEdgeSize = FMath::Min(3 * NumTris, 3 * FMath::Max(NumVerts,2) - 6);
	Edges.Empty(maxEdgeSize);
	for (int i = 0; i < NumVerts; i++)
	{
		AppendConnectedEdges(&Verts[i], Edges);
	}

	// Guessed wrong on num edges. Array was resized so fix up pointers.

	if (Edges.Num() > maxEdgeSize)
	{
	
		for (int32 i = 0; i < Edges.Num(); ++i)
		{
			SimpEdgeType& edge = Edges[i];
			edge.next = &edge;
			edge.prev = &edge;
		}
	}

}

void SkeletalSimplifier::FSimplifierMeshManager::AppendConnectedEdges(const SimpVertType* Vert, TArray< SimpEdgeType >& Edges)
{


	// Need to cast the vert - but the method we are calling on it really should be const..
	SimpVertType* V = const_cast<SimpVertType*>(Vert);

	checkSlow(V->adjTris.Num() > 0);

	TArray< SimpVertType*, TInlineAllocator<64> > adjVerts;
	V->FindAdjacentVerts(adjVerts);

	SimpVertType* v0 = V;
	for (SimpVertType* v1 : adjVerts)
	{
		if (v0 < v1)
		{
			// add edge
			Edges.AddDefaulted();
			SimpEdgeType& edge = Edges.Last();
			edge.v0 = v0;
			edge.v1 = v1;
		}
	}
}


void SkeletalSimplifier::FSimplifierMeshManager::GroupEdges(TArray< SimpEdgeType >& Edges)
{
	FHashTable HashTable(1 << FMath::Min(16u, FMath::FloorLog2(Edges.Num() / 2)), Edges.Num());

	TArray<uint32> HashValues;
	ResizeArray(HashValues, Edges.Num());

	
	{
		for (int32 i = 0, I = Edges.Num(); i < I; ++i)
		{
			uint32 Hash0 = HashPoint((FVector)Edges[i].v0->GetPos());
			uint32 Hash1 = HashPoint((FVector)Edges[i].v1->GetPos());
			HashValues[i] = Murmur32({ FMath::Min(Hash0, Hash1), FMath::Max(Hash0, Hash1) });
		}

	}

	for (int32 i = 0, IMax = Edges.Num(); i < IMax; ++i)
	{
		HashTable.Add(HashValues[i], i);
	}

	const bool bTestCoincidentBones = !bWeldBonesIfSamePos;

	for (int32 i = 0, IMax = Edges.Num(); i < IMax; ++i)
	{
		// already grouped
		if (Edges[i].next != &Edges[i])
		{
			continue;
		}

		// find any matching edges
		uint32 Hash = HashValues[i];
		for (uint32 j = HashTable.First(Hash); HashTable.IsValid(j); j = HashTable.Next(j))
		{
			SimpEdgeType* e1 = &Edges[i];
			SimpEdgeType* e2 = &Edges[j];

			if (e1 == e2)
				continue;

			bool m1 = 
			    (e1->v0 == e2->v0  || IsCoincident(e1->v0, e2->v0, bTestCoincidentBones) ) &&
				(e1->v1 == e2->v1  || IsCoincident(e1->v1, e2->v1, bTestCoincidentBones) );
			// same edge but with reversed orientation
			bool m2 =
				(e1->v0 == e2->v1 || IsCoincident(e1->v0, e2->v1, bTestCoincidentBones)) &&
				(e1->v1 == e2->v0 || IsCoincident(e1->v1, e2->v0, bTestCoincidentBones));



			// backwards
			if (m2)
			{
				Swap(e2->v0, e2->v1);
			}

			

			// link
			if (m1 || m2)
			{
		
				checkSlow(e2->next == e2);
				checkSlow(e2->prev == e2);

				e2->next = e1->next;
				e2->prev = e1;
				e2->next->prev = e2;
				e2->prev->next = e2;
			}
		}
	}
}

void SkeletalSimplifier::FSimplifierMeshManager::GetCoincidentVertGroups(VertPtrArray& CoincidentVertGroups)
{
	for (int32 vId = 0; vId < NumSrcVerts; ++vId)
	{
		SimpVertType* Vert = &VertArray[vId];

		// Removed vert
		if (Vert->TestFlags(SIMP_REMOVED))
		{
			continue;
		}

		// Single vert
		if (Vert->next == Vert && Vert->prev == Vert)
		{
			continue;
		}

		// Find and add the max vert in this group.
		{
			SimpVertType* tmp = Vert;
			SimpVertType* maxVert = Vert;
			while (tmp->next != Vert)
			{
				tmp = tmp->next;
				if (tmp > maxVert && !tmp->TestFlags(SIMP_REMOVED)) 
				{
					maxVert = tmp;
				}
			}

			CoincidentVertGroups.AddUnique(maxVert);
		}
	}
}

void  SkeletalSimplifier::FSimplifierMeshManager::WeldNonSplitBasicAttributes(EVtxElementWeld WeldType)
{

	// Gather the split-vertex groups.  
	VertPtrArray CoincidentVertGroups;
	GetCoincidentVertGroups(CoincidentVertGroups);

	// For each split group, weld the attributes that have the same element ID
	int32 NumCoincidentVertGroups = CoincidentVertGroups.Num();

	for (int32 i = 0; i < NumCoincidentVertGroups; ++i)
	{
		SimpVertType* HeadVert = CoincidentVertGroups[i];

		if (!HeadVert) continue;

		// Get the verts that are in this group.
		VertPtrArray VertGroup;
		GetVertsInGroup(*HeadVert, VertGroup);

		int32  NumVertsInGroup = VertGroup.Num();

		// reject any groups that weren't really split.
		if (NumVertsInGroup < 2) continue;


		// functor that partitions attributes by attribute ID and welds attributes in a partition to the average value.
		auto Weld = [&VertGroup, NumVertsInGroup](auto& IDAccessor, auto& ValueAccessor, auto ZeroValue)
		{
			// container used to sort coincident vert by ID.  
			// used to find groups with same ID to weld together.

			TArray<FVertAndID, TInlineAllocator<5>> VertAndIDArray;
			for (SimpVertType* v : VertGroup)
			{
				VertAndIDArray.Emplace(v, IDAccessor(v));
			}
			// sort by ID
			VertAndIDArray.Sort([](const FVertAndID& A, const FVertAndID& B)->bool {return A.ID < B.ID; });

			// find and process the partitions.

			int32 PartitionStart = 0;
			while (PartitionStart < NumVertsInGroup)
			{
				auto AveValue = ZeroValue;
				int32 PartitionElID = VertAndIDArray[PartitionStart].ID;
				int32 PartitionEnd = NumVertsInGroup;
				for (int32 n = PartitionStart; n < NumVertsInGroup; ++n)
				{
					if (VertAndIDArray[n].ID == PartitionElID)
					{
						AveValue += ValueAccessor(VertAndIDArray[n].SrcVert);
					}
					else
					{
						PartitionEnd = n;
						break;
					}
				}
				AveValue /= (PartitionEnd - PartitionStart);
				for (int32 n = PartitionStart; n < PartitionEnd; ++n)
				{
					ValueAccessor(VertAndIDArray[n].SrcVert) = AveValue;
				}

				PartitionStart = PartitionEnd;
			}

		};

		// Weld Normals with the same NormalID.
		if (WeldType == EVtxElementWeld::Normal)
		{
			auto NormalIDAccessor = [](SimpVertType* SimpVert)->int32
			{
				return SimpVert->vert.BasicAttributes.ElementIDs.NormalID;
			};
			auto NormalValueAccessor = [](SimpVertType* SimpVert)->FVector3f&
			{
				return SimpVert->vert.BasicAttributes.Normal;
			};

			FVector3f ZeroValue(0, 0, 0);
			Weld(NormalIDAccessor, NormalValueAccessor, ZeroValue);
		}
		// Weld Tangents with same TangentID
		if (WeldType == EVtxElementWeld::Tangent)
		{

			auto TangentIDAccessor = [](SimpVertType* SimpVert)->int32
			{
				return SimpVert->vert.BasicAttributes.ElementIDs.TangentID;
			};
			auto TangentValueAccessor = [](SimpVertType* SimpVert)->FVector3f&
			{
				return SimpVert->vert.BasicAttributes.Tangent;
			};

			FVector3f ZeroValue(0, 0, 0);
			Weld(TangentIDAccessor, TangentValueAccessor, ZeroValue);
		}
		// Weld BiTangent with same BiTangentID
		if (WeldType == EVtxElementWeld::BiTangent)
		{
			auto BiTangentIDAccessor = [](SimpVertType* SimpVert)->int32
			{
				return SimpVert->vert.BasicAttributes.ElementIDs.BiTangentID;
			};
			auto BiTangentValueAccessor = [](SimpVertType* SimpVert)->FVector3f&
			{
				return SimpVert->vert.BasicAttributes.BiTangent;
			};

			FVector3f ZeroValue(0, 0, 0);
			Weld(BiTangentIDAccessor, BiTangentValueAccessor, ZeroValue);
		}
		// Weld Color with same ColorID
		if (WeldType == EVtxElementWeld::Color)
		{

			auto ColorIDAccessor = [](SimpVertType* SimpVert)->int32
			{
				return SimpVert->vert.BasicAttributes.ElementIDs.ColorID;
			};
			auto ColorValueAccessor = [](SimpVertType* SimpVert)->FLinearColor&
			{
				return SimpVert->vert.BasicAttributes.Color;
			};

			FLinearColor ZeroValue = FLinearColor::Transparent;
			Weld(ColorIDAccessor, ColorValueAccessor, ZeroValue);
		}
		// Weld UVs with same TexCoordsID
		if (WeldType == EVtxElementWeld::UV)
		{
			int32 NumTexCoords = SkeletalSimplifier::BasicAttrArray::NumUVs;
			for (int32 t = 0; t < NumTexCoords; ++t)
			{

				auto TexCoordIDAccessor = [t](SimpVertType* SimpVert)->int32
				{
					return SimpVert->vert.BasicAttributes.ElementIDs.TexCoordsID[t];
				};
				auto TexCoordValueAccessor = [t](SimpVertType* SimpVert)->FVector2f&
				{
					return SimpVert->vert.BasicAttributes.TexCoords[t];
				};

				FVector2f ZeroValue(0, 0);
				Weld(TexCoordIDAccessor, TexCoordValueAccessor, ZeroValue);
			}
		}

		// After welding, need to "correct" the vert to make sure the vector attributes are normalized etc.
		for (SimpVertType* v : VertGroup)
		{
			v->vert.Correct();
		}
	}
}


// @todo, this shares a lot of code with GroupEdges - they should be unified..
void SkeletalSimplifier::FSimplifierMeshManager::RebuildEdgeLinkLists(EdgePtrArray& CandidateEdgePtrArray)
{
	const uint32 NumEdges = CandidateEdgePtrArray.Num();
	// Fix edge groups - when one edge of a triangle collapses the opposing edges end up merging.. this accounts for that i think.
	{
		FHashTable HashTable(128, NumEdges);

		// ungroup edges
		for (uint32 i = 0; i < NumEdges; ++i)
		{
			SimpEdgeType* edge = CandidateEdgePtrArray[i];

			if (edge->TestFlags(SIMP_REMOVED))
				continue;

			edge->next = edge;
			edge->prev = edge;
		}

		// Hash Edges.
		for (uint32 i = 0; i < NumEdges; ++i)
		{
			SimpEdgeType& edge = *CandidateEdgePtrArray[i];

			if (edge.TestFlags(SIMP_REMOVED))
				continue;

			HashTable.Add(HashEdgePosition(edge), i);
		}

		const bool bTestCoincidentBones = !bWeldBonesIfSamePos;

		// regroup edges
		for (uint32 i = 0; i < NumEdges; ++i)
		{
			SimpEdgeType* edge = CandidateEdgePtrArray[i];

			if (edge->TestFlags(SIMP_REMOVED))
				continue;

			// already grouped
			if (edge->next != edge)
				continue;

			// find any matching edges
			uint32 hash = HashEdgePosition(*edge);
			SimpEdgeType* e1 = edge;
			for (uint32 j = HashTable.First(hash); HashTable.IsValid(j); j = HashTable.Next(j))
			{

				SimpEdgeType* e2 = CandidateEdgePtrArray[j];

				if (e1 == e2)
					continue;

				bool m1 = 
					 (e1->v0 == e2->v0 && 
					  e1->v1 == e2->v1)
					  ||
					  ( IsCoincident(e1->v0, e2->v0, bTestCoincidentBones) &&
					    IsCoincident(e1->v1, e2->v1, bTestCoincidentBones) );
				// same edge but with reversed orientation
				bool m2 =
					(e1->v0 == e2->v1 &&
					 e1->v1 == e2->v0)
					||
				    ( IsCoincident(e1->v0, e2->v1, bTestCoincidentBones) &&
					  IsCoincident(e1->v1, e2->v0, bTestCoincidentBones));

				// backwards
				if (m2)
				Swap(e2->v0, e2->v1);

				// link
				if (m1 || m2)
				{
					checkSlow(e2->next == e2);
					checkSlow(e2->prev == e2);

					e2->next = e1->next;
					e2->prev = e1;
					e2->next->prev = e2;
					e2->prev->next = e2;
				}
			}
		}
	}
}


void SkeletalSimplifier::FSimplifierMeshManager::VisitEdges(TFunctionRef<void(SimpVertType*, SimpVertType*, int32)> EdgeVisitor)
{

	TArray< SimpVertType*, TInlineAllocator<64> > adjVerts;
	if (NumSrcVerts == 0 || NumSrcTris == 0)
	{
		//Avoid trying to compute an empty mesh
		return;
	}

	// clear the mark2 flags. We use these to determine if we have visited a vert group.
	for (int32 i = 0; i < NumSrcVerts; ++i)
	{
		SimpVertType* v0 = &VertArray[i];
		v0->DisableFlags(SIMP_MARK2);
	}

	for (int i = 0; i < NumSrcVerts; i++)
	{

		SimpVertType* v0 = &VertArray[i];
		checkSlow(v0 != NULL);
		check(v0->adjTris.Num() > 0);

		// we have already visited this vertex group
		if (v0->TestFlags(SIMP_MARK2))
		{
			continue;
		}

		if (v0->TestFlags(SIMP_REMOVED))
		{
			continue;
		}

		//Find all the verts that are adjacent to any vert in this group.
		adjVerts.Reset();
		SimpVertType* v0Smallest = v0;
		{
			SimpVertType* v = v0;
			do {
				for (TriIterator triIter = v->adjTris.Begin(); triIter != v->adjTris.End(); ++triIter)
				{
					for (int j = 0; j < 3; j++)
					{
						SimpVertType* TriVert = (*triIter)->verts[j];
						if (TriVert != v)
						{
							adjVerts.AddUnique(TriVert);
						}
					}
				}
				v = v->next;
				if (v0Smallest > v)
				{
					v0Smallest = v;
				}
			} while (v != v0);
		}

		for (SimpVertType* v1 : adjVerts)
		{
			// visit edges that are incoming to this vertex group
			// note, we may end up visiting a few edges twice.
			if (v0Smallest < v1)
			{

				// set if this edge is boundary
				// find faces that share v0 and v1
				v0->EnableAdjTriFlagsGroup(SIMP_MARK1);
				v1->DisableAdjTriFlagsGroup(SIMP_MARK1);

				int32 AdjFaceCount = 0;
				SimpVertType* vert = v0;
				do
				{
					for (TriIterator j = vert->adjTris.Begin(); j != vert->adjTris.End(); ++j)
					{
						SimpTriType* tri = *j;
						AdjFaceCount += tri->TestFlags(SIMP_MARK1) ? 0 : 1;
					}
					vert = vert->next;
				} while (vert != v0);

				// reset v0-group flag.
				v0->DisableAdjTriFlagsGroup(SIMP_MARK1);

				// process this edge.
				EdgeVisitor(v0, v1, AdjFaceCount);
			}
		}

		// visited this vert and all the incoming edges.
		v0->EnableFlagsGroup(SIMP_MARK2);
	}

	for (int32 i = 0; i < NumSrcVerts; ++i)
	{
		SimpVertType* v0 = &VertArray[i];
		v0->DisableFlags(SIMP_MARK2);
	}
}


//[TODO]  convert this to use the VisitEdges method.
void SkeletalSimplifier::FSimplifierMeshManager::FlagBoundary(const ESimpElementFlags Flag)
{ 

	TArray< SimpVertType*, TInlineAllocator<64> > adjVerts;
	if (NumSrcVerts == 0 || NumSrcTris == 0)
	{
		//Avoid trying to compute an empty mesh
		return;
	}

	for (int i = 0; i < NumSrcVerts; i++)
	{

		SimpVertType* v0 = &VertArray[i];
		checkSlow(v0 != NULL);
		check(v0->adjTris.Num() > 0);

		// not sure if this test is valid.  
		if (v0->TestFlags(Flag))
		{
			// we must have visited this vert already in a vert group
			continue;
		}


		//Find all the verts that are adjacent to any vert in this group.
		adjVerts.Reset();
		//the scope below replaces  v0->FindAdjacentVertsGroup(adjVerts);
		{
			SimpVertType* v = v0;
			do {
				for (TriIterator triIter = v->adjTris.Begin(); triIter != v->adjTris.End(); ++triIter)
				{
					for (int j = 0; j < 3; j++)
					{
						SimpVertType* TriVert = (*triIter)->verts[j];
						if (TriVert != v)
						{
							adjVerts.AddUnique(TriVert);
						}
					}
				}
				v = v->next;
			} while (v != v0);
		}

		for (SimpVertType* v1 : adjVerts)
		{
			if (v0 < v1)
			{

				// set if this edge is boundary
				// find faces that share v0 and v1
				v0->EnableAdjTriFlagsGroup(SIMP_MARK1);
				v1->DisableAdjTriFlagsGroup(SIMP_MARK1);

				int faceCount = 0;
				SimpVertType* vert = v0;
				do
				{
					for (TriIterator j = vert->adjTris.Begin(); j != vert->adjTris.End(); ++j)
					{
						SimpTriType* tri = *j;
						faceCount += tri->TestFlags(SIMP_MARK1) ? 0 : 1;
					}
					vert = vert->next;
				} while (vert != v0);

				// reset v0-group flag.
				v0->DisableAdjTriFlagsGroup(SIMP_MARK1);

				if (faceCount == 1)
				{
					// only one face on this edge
					v0->EnableFlagsGroup(Flag);
					v1->EnableFlagsGroup(Flag);
				}
			}
		}
	}
}

void SkeletalSimplifier::FSimplifierMeshManager::FlagBoxCorners(const ESimpElementFlags Flag)
{


	TArray<int32> VisitedMask;
	VisitedMask.AddZeroed(NumSrcVerts);

	for (int32 i = 0; i < NumSrcVerts; i++)
	{

		if (VisitedMask[i] != 0)
		{
			continue;
		}

		// Collect all the face normals associated with this vertgroup

		TArray<FVector, TInlineAllocator<6>> FaceNormals;

		SimpVertType* Vert = GetVertPtr(i);
		SimpVertType* seedVert = Vert;
		do {

			for (TriIterator triIter = Vert->adjTris.Begin(); triIter != Vert->adjTris.End(); ++triIter)
			{
				bool bIsDuplicate = false;

				SimpTriType* tri = *triIter;

				FVector Nrml = (FVector)tri->GetNormal();
			
				for (int32 fnIdx = 0; fnIdx < FaceNormals.Num(); ++fnIdx)
				{
					FVector ExistingNormal = FaceNormals[fnIdx];
					ExistingNormal.Normalize();
					float DotValue = FVector::DotProduct(ExistingNormal, Nrml);

					if (1.f - DotValue < 0.133975f) // 30 degrees.
					{
						// we already have this vector.
						// could be a corner
						bIsDuplicate = true;
						FaceNormals[fnIdx] += Nrml;
						continue;

					}

				}

				if (!bIsDuplicate)
				{
					FaceNormals.Add(Nrml);
				}
			}

			// mark as visited.

			uint32 Idx = GetVertIndex(Vert);
			VisitedMask[Idx] = 1;

			Vert = Vert->next;
		} while (Vert != seedVert);



		int32 FaceCount = FaceNormals.Num();

		if ( FaceNormals.Num() == 3 )
		{
			FVector& A = FaceNormals[0];
			FVector& B = FaceNormals[1];
			FVector& C = FaceNormals[2];

			A.Normalize();
			B.Normalize();
			C.Normalize();

			float AdotB = FVector::DotProduct(A, B);
			float BdotC = FVector::DotProduct(B, C);
			float AdotC = FVector::DotProduct(A, C);

			if (FMath::Abs(AdotB) < 0.259f && FMath::Abs(BdotC) < 0.259f && FMath::Abs(AdotC) < 0.259f) // 15-degrees off normal
			{
				Vert->EnableFlagsGroup(Flag);
			}
			
		}

	
	}

}

void SkeletalSimplifier::FSimplifierMeshManager::FlagEdges(const TFunction<bool(const SimpVertType*, const SimpVertType*)> IsDifferent, const ESimpElementFlags Flag)
{
	int32 NumEdges = EdgeArray.Num();

	for (int32 i = 0; i < NumEdges; ++i)
	{
		SimpEdgeType& Edge = EdgeArray[i];
		if (IsDifferent(Edge.v0, Edge.v1))
		{
			Edge.EnableFlags(Flag);
			Edge.v0->EnableFlags(Flag);
			Edge.v1->EnableFlags(Flag);
		}
	}

	for (int i = 0; i < NumSrcVerts; i++)
	{
		SimpVertType* v = &VertArray[i];
		SimpVertType* v1 = v;

		if (v1 == nullptr || v1->TestFlags(Flag))
		{
			continue;
		}
		// only one vert in this group
		if (v1->next == v1)
		{
			continue;
		}
		
		bool bAddedFlag = false;
		do {

			if (IsDifferent(v, v->next))
			{
				// we only need to mark one of the vertices in this vertex group since the lock state will be propagated to the others.
				v->EnableFlags(Flag);
				bAddedFlag = true;
			}
			v = v->next;
		} while (v != v1);

		// add the locked state to all the vertices in this group.
		if (bAddedFlag)
		{
			v = v1;
			do {
				v->EnableFlags(Flag);
				v = v->next;
			} while (v != v1);

		}
		
	}
}

void SkeletalSimplifier::FSimplifierMeshManager::GetAdjacentTopology(const SimpVertType* VertPtr,
	TriPtrArray& DirtyTris, VertPtrArray& DirtyVerts, EdgePtrArray& DirtyEdges)
{
	// need this cast because the const version is missing on the vert..

	SimpVertType* v = const_cast<SimpVertType*>(VertPtr);

	// Gather pointers to all the triangles that share this vert.

	// Update all tris touching collapse edge.
	for (TriIterator triIter = v->adjTris.Begin(); triIter != v->adjTris.End(); ++triIter)
	{
		DirtyTris.AddUnique(*triIter);
	}

	// Gather all verts that are adjacent to this one.

	TArray< SimpVertType*, TInlineAllocator<64> > adjVerts;
	v->FindAdjacentVerts(adjVerts);


	// Gather verts that are adjacent to VertPtr
	for (int i = 0, Num = adjVerts.Num(); i < Num; i++)
	{
		DirtyVerts.AddUnique(adjVerts[i]);
	}



	// Gather verts that are adjacent to VertPtr
	for (int i = 0, Num = adjVerts.Num(); i < Num; i++)
	{
		adjVerts[i]->EnableFlags(SIMP_MARK2);
	}

	// update the costs of all edges connected to any face adjacent to v
	for (int i = 0, iMax = adjVerts.Num(); i < iMax; ++i)
	{
		SimpVertType* AdjVert = adjVerts[i];
		AdjVert->EnableAdjVertFlags(SIMP_MARK1);

		for (TriIterator triIter = AdjVert->adjTris.Begin(); triIter != AdjVert->adjTris.End(); ++triIter)
		{
			SimpTriType* tri = *triIter;
			for (int k = 0; k < 3; k++)
			{
				SimpVertType* vert = tri->verts[k];
				if (vert->TestFlags(SIMP_MARK1) && !vert->TestFlags(SIMP_MARK2) && vert != AdjVert)
				{
					SimpEdgeType* edge = FindEdge(AdjVert, vert);
					DirtyEdges.AddUnique(edge);
				}
				vert->DisableFlags(SIMP_MARK1);
			}
		}
		AdjVert->DisableFlags(SIMP_MARK2);
	}


}

void SkeletalSimplifier::FSimplifierMeshManager::GetAdjacentTopology(const SimpEdgeType& GroupedEdge,
	TriPtrArray& DirtyTris, VertPtrArray& DirtyVerts, EdgePtrArray& DirtyEdges)
{
	// Find the parts of the mesh that will be 'dirty' after the 
	// edge collapse. 

	const SimpVertType* v = GroupedEdge.v0;
	do {
		GetAdjacentTopology(v, DirtyTris, DirtyVerts, DirtyEdges);
		v = v->next;
	} while (v != GroupedEdge.v0);

	v = GroupedEdge.v1;
	do {
		GetAdjacentTopology(v, DirtyTris, DirtyVerts, DirtyEdges);
		v = v->next;
	} while (v != GroupedEdge.v1);

}

int32 SkeletalSimplifier::FSimplifierMeshManager::RemoveEdgeIfInvalid(EdgePtrArray& CandidateEdges, IdxArray& RemovedEdgeIdxArray)
{

	const uint32 NumCandidateEdges = CandidateEdges.Num();
	for (uint32 i = 0; i < NumCandidateEdges; ++i)
	{
	
		SimpEdgeType* EdgePtr = CandidateEdges[i];

		// Edge has already been removed..
		if (!EdgePtr) continue;

		
		// Verify the edge has an adjacent face.
		auto HasAdjacentFace = [](SimpEdgeType* e)->bool
		{
			// Verify that this is truly an edge of a triangle

			e->v0->EnableAdjVertFlags(SIMP_MARK1);
			e->v1->DisableAdjVertFlags(SIMP_MARK1);

			if (e->v0->TestFlags(SIMP_MARK1))
			{
				// Invalid edge results from collapsing a bridge tri
				// There are no actual triangles connecting these verts
				e->v0->DisableAdjVertFlags(SIMP_MARK1);

				return false;
			}

			return true;
		};

		// remove invalid faces from the hash, edge group and flag.
		if ( IsInvalid(EdgePtr) || !HasAdjacentFace(EdgePtr) ) // one of the verts touches zero triangles
		{
			// unlinks the edge from the edge group, remove from edge hash, add removed flag.

			const uint32 Idx = RemoveEdge(*EdgePtr); 

			// Record the index of the edge we remove.

			if (Idx < INVALID_EDGE_ID)
			{
				RemovedEdgeIdxArray.AddUnique(Idx); 
			}
			CandidateEdges[i] = NULL;
		}
	}

	return RemovedEdgeIdxArray.Num();
}

uint32 SkeletalSimplifier::FSimplifierMeshManager::RemoveEdge(const SimpVertType* VertAPtr, const SimpVertType* VertBPtr)
{
	auto HashAndIdx = GetEdgeHashPair(VertAPtr, VertBPtr);

	uint32 Idx = HashAndIdx.Key;
	// Early out if this edge doesn't exist.
	if (Idx == INVALID_EDGE_ID)
	{
		return Idx;
	}

	SimpEdgeType& Edge = EdgeArray[Idx];
	if (Edge.TestFlags(SIMP_REMOVED))
	{
		Idx = INVALID_EDGE_ID;
	}
	else
	{
		// mark as removed
		Edge.EnableFlags(SIMP_REMOVED);
		EdgeVertIdHashMap.Remove(HashAndIdx.Value, Idx);

	}

	// remove this edge from its edge group
	Edge.prev->next = Edge.next;
	Edge.next->prev = Edge.prev;

	Edge.next = &Edge;
	Edge.prev = &Edge;

	// return the Idx
	return Idx;
}

uint32 SkeletalSimplifier::FSimplifierMeshManager::RemoveEdge(SimpEdgeType& Edge)
{
	// remove this edge from its edge group
	Edge.prev->next = Edge.next;
	Edge.next->prev = Edge.prev;

	Edge.next = &Edge;
	Edge.prev = &Edge;

	uint32 Idx = GetEdgeIndex(&Edge);

	if (Edge.TestFlags(SIMP_REMOVED))
	{
		Idx = INVALID_EDGE_ID;
	}
	else
	{

		// mark as removed
		Edge.EnableFlags(SIMP_REMOVED);

		uint32 Hash = HashEdge(Edge.v0, Edge.v1);

		EdgeVertIdHashMap.Remove(Hash, Idx);
	}
	// return the Idx
	return Idx;
}

uint32 SkeletalSimplifier::FSimplifierMeshManager::ReplaceVertInEdge(const SimpVertType* VertAPtr, const SimpVertType* VertBPtr, SimpVertType* VertAprimePtr)
{

	TPair<uint32, uint32> HashAndIdx = GetEdgeHashPair(VertAPtr, VertBPtr);
	const uint32 Idx = HashAndIdx.Key;
	const uint32 HashValue = HashAndIdx.Value;

	checkSlow(Idx != INVALID_EDGE_ID);
	SimpEdgeType* edge = &EdgeArray[Idx];

	EdgeVertIdHashMap.Remove(HashValue, Idx);

	EdgeVertIdHashMap.Add(HashEdge(VertAprimePtr, VertBPtr), Idx);

	if (edge->v0 == VertAPtr)
		edge->v0 = VertAprimePtr;
	else
		edge->v1 = VertAprimePtr;

	return Idx;
}

int32 SkeletalSimplifier::FSimplifierMeshManager::RemoveIfDegenerate(TriPtrArray& CandidateTrisPtrArray)
{
	int32 NumRemoved = 0;
	// remove degenerate triangles
	// not sure why this happens
	for (SimpTriType* CandidateTriPtr : CandidateTrisPtrArray)
	{
		if (CandidateTriPtr->TestFlags(SIMP_REMOVED))
			continue;


		const FVector& p0 = (FVector)CandidateTriPtr->verts[0]->GetPos();
		const FVector& p1 = (FVector)CandidateTriPtr->verts[1]->GetPos();
		const FVector& p2 = (FVector)CandidateTriPtr->verts[2]->GetPos();
		const FVector n = (p2 - p0) ^ (p1 - p0);

		if (n.SizeSquared() == 0.0f)
		{
			NumRemoved++;
			CandidateTriPtr->EnableFlags(SIMP_REMOVED);

			// remove references to tri
			for (int j = 0; j < 3; j++)
			{
				SimpVertType* vert = CandidateTriPtr->verts[j];
				vert->adjTris.Remove(CandidateTriPtr);
				// orphaned verts are removed below
			}
		}
	}

	ReducedNumTris -= NumRemoved;
	return NumRemoved;
}



int32 SkeletalSimplifier::FSimplifierMeshManager::RemoveDegenerateTris()
{

	TriPtrArray TriPtrs;
	ResizeArray(TriPtrs, NumSrcTris);

	{
		for (int32 i = 0, IMax = NumSrcTris; i < IMax; ++i)
		{
			TriPtrs[i] = &TriArray[i];
		}
	}

	return RemoveIfDegenerate(TriPtrs);
}

int32 SkeletalSimplifier::FSimplifierMeshManager::RemoveIfDegenerate(VertPtrArray& CandidateVertPtrArray)
{
	int32 NumRemoved = 0;
	// remove orphaned verts
	for (SimpVertType* VertPtr : CandidateVertPtrArray)
	{
		if (VertPtr->TestFlags(SIMP_REMOVED))
			continue;
		
		if (VertPtr->adjTris.Num() == 0)
		{
			NumRemoved++;
			VertPtr->EnableFlags(SIMP_REMOVED);
		
			// ungroup
			VertPtr->prev->next = VertPtr->next;
			VertPtr->next->prev = VertPtr->prev;
			VertPtr->next = VertPtr;
			VertPtr->prev = VertPtr;
		}
	}

	ReducedNumVerts -= NumRemoved;
	return NumRemoved;
}

int32 SkeletalSimplifier::FSimplifierMeshManager::RemoveDegenerateVerts()
{

	VertPtrArray VertPtrs;
	ResizeArray(VertPtrs, NumSrcVerts);

	{

		for (int32 i = 0, IMax = NumSrcVerts; i < IMax; ++i)
		{
			VertPtrs[i] = &VertArray[i];
		}
	}
	return RemoveIfDegenerate(VertPtrs);
}

int32 SkeletalSimplifier::FSimplifierMeshManager::RemoveIfDegenerate(EdgePtrArray& CandidateEdges, IdxArray& RemovedEdgeIdxArray)
{
	const uint32 NumCandidateEdges = CandidateEdges.Num();

	// add all grouped edges
	for (uint32 i = 0; i < NumCandidateEdges; i++)
	{
		SimpEdgeType* edge = CandidateEdges[i];

		if (edge->TestFlags(SIMP_REMOVED))
			continue;

		SimpEdgeType* e = edge;
		do {
			CandidateEdges.AddUnique(e);
			e = e->next;
		} while (e != edge);
	}

	// remove dead edges from our edge hash.
	for (uint32 i = 0, Num = CandidateEdges.Num(); i < Num; i++)
	{
		SimpEdgeType* edge = CandidateEdges[i];

		if (edge->TestFlags(SIMP_REMOVED))
			continue;
	
		if (edge->v0 == edge->v1)
		{
			edge->EnableFlags(SIMP_REMOVED); // djh 8/3/18.  not sure why this happens
											 
			uint32 Idx = RemoveEdge(*edge);
			if (Idx < INVALID_EDGE_ID)
			{
				RemovedEdgeIdxArray.AddUnique(Idx);
			}
		}
		else if (edge->v0->TestFlags(SIMP_REMOVED) ||
			     edge->v1->TestFlags(SIMP_REMOVED))
		{
			
			uint32 Idx = RemoveEdge(*edge);
			if (Idx < INVALID_EDGE_ID)
			{
				RemovedEdgeIdxArray.AddUnique(Idx);
			}
		}
	}

	return RemovedEdgeIdxArray.Num();
}

void SkeletalSimplifier::FSimplifierMeshManager::UpdateVertexAttriuteIDs(EdgePtrArray& InCoincidentEdges)
{

	// some of the edges may be null / removed.  Need to filter these out.

	EdgePtrArray CoincidentEdges;
	for (SimpEdgeType* EdgePtr : InCoincidentEdges)
	{
		if (!EdgePtr || !EdgePtr->v0 || !EdgePtr->v1 || IsRemoved(EdgePtr))
		{
			continue;
		}

		CoincidentEdges.Add(EdgePtr);
	}


	int32 NumCoincidentEdges = CoincidentEdges.Num();

	if (NumCoincidentEdges == 0)
	{
		return;
	}

	if (NumCoincidentEdges > 2)
	{
		// this is the collapse of a non-manifold edge.  Not going to track the element Ids in this badness. 
		return;
	}

	// Update the Attribute element IDs on the loose v0 verts (since v0 collapses onto v1)
	// Loose verts updated any non-split element IDs to those of the v1.
	if (NumCoincidentEdges == 1)
	{

		SimpEdgeType* EdgePtr = CoincidentEdges[0];


		auto v0IDs = EdgePtr->v0->vert.BasicAttributes.ElementIDs;
		auto v1IDs = EdgePtr->v1->vert.BasicAttributes.ElementIDs;

		// loop over the v0 loose verts and update any non-split element IDs to v1ID.
		SimpVertType* v0 = EdgePtr->v0;
		SimpVertType* vCurrent = v0;
		while (vCurrent->next != v0)
		{
			vCurrent = vCurrent->next;
			// create a mask that is zero where the elements are the same as the v0 elements.
			// those elements should be merged with their v1 counterparts, retaining the v1 ids. 
			auto MaskIDs = vCurrent->vert.BasicAttributes.ElementIDs - v0IDs;

			// copy element ids from v1IDs to vCurrent where the mask is off (zero)
			vCurrent->vert.BasicAttributes.ElementIDs.MaskedCopy(MaskIDs, v1IDs);
		}

	}

	if (NumCoincidentEdges == 2)
	{
		// the edge is split.  There are two verts at each location
		SimpVertType* v0[2];
		SimpVertType* v1[2];

		for (int i = 0; i < 2; ++i)
		{
			SimpEdgeType* EdgePtr = CoincidentEdges[i];

			v0[i] = EdgePtr->v0;
			v1[i] = EdgePtr->v1;
		}

		// force attrs that are split on v0 to be split on v1.  If an attribute is split on v0 and not on v1
		// then this copies the ids from v0 to v1 for this attribute.
		// Why do this?  When an attribute seam starts on a mesh, one vertex may have a split and the other not.
		// This can even happen in the middle of a seam, for example to neighboring UV patches might agree at a single
		// point.
		//  An Example helps:  Say color ID is split on the v0 vertex but not the v1 vertex
		//      v0[0] ColorID = 7,  v0[1] ColorID =8       v1[0] ColorID = 11, v1[1] ColorID = 11    
		//
		//      this should result in copying that split to the v1 vertex.  =>  v1[0] ColorID =7  v1[1] ColorID = 8
		//
		if (v1[0] != v1[1] && v0[0] != v0[1]) // only apply this to the case of two distinct edges.
		{
			// Identify the elements that are not split on the v0 vertex (zeros of mask)
			auto v0MaskIDs = v0[0]->vert.BasicAttributes.ElementIDs - v0[1]->vert.BasicAttributes.ElementIDs;

			// Identify the elements that are not split on the v1 vertex ( zero of mask )
			auto v1MaskIDs = v1[0]->vert.BasicAttributes.ElementIDs - v1[1]->vert.BasicAttributes.ElementIDs;

			// Copy values from v0 IDs where v1Mask is off (zero) and v0 mask is on (non-zero)
			v1[0]->vert.BasicAttributes.ElementIDs.MaskedCopy(v1MaskIDs, v0MaskIDs, v0[0]->vert.BasicAttributes.ElementIDs);
			v1[1]->vert.BasicAttributes.ElementIDs.MaskedCopy(v1MaskIDs, v0MaskIDs, v0[1]->vert.BasicAttributes.ElementIDs);
		}

		// Update the Attribute element IDs on the loose v0 verts (since v0 collapses onto v1)
		// Loose verts updated any non-split element IDs to those of the v1.
		for (int i = 0; i < 2; ++i)
		{
			SimpEdgeType* EdgePtr = CoincidentEdges[i];
			auto v0IDs = v0[i]->vert.BasicAttributes.ElementIDs;
			auto v1IDs = v1[i]->vert.BasicAttributes.ElementIDs;

			SimpVertType* vCurrent = v0[i];
			while (vCurrent->next != v0[i])
			{
				vCurrent = vCurrent->next;
				if (vCurrent != v0[0] && vCurrent != v0[1]) // only look at the v0 loose verts
				{
					// create a mask where the elements are the same
					auto MaskIDs = vCurrent->vert.BasicAttributes.ElementIDs - v0IDs;

					// copy elements from V1IDs to tmp where the mask is off (zero)
					vCurrent->vert.BasicAttributes.ElementIDs.MaskedCopy(MaskIDs, v1IDs);
				}
			}
		}

	}

}

bool SkeletalSimplifier::FSimplifierMeshManager::CollapseEdge(SimpEdgeType * EdgePtr, IdxArray& RemovedEdgeIdxArray)
{
	SimpVertType* v0 = EdgePtr->v0;
	SimpVertType* v1 = EdgePtr->v1;

	// Collapse the edge uv by moving vertex v0 onto v1
	checkSlow(v0 && v1);
	checkSlow(EdgePtr == FindEdge(v0, v1));

	checkSlow(v0->adjTris.Num() > 0);
	checkSlow(v1->adjTris.Num() > 0);

	// Because another edge in the same edge group may share a vertex with this edge
	// and it might have already been collapsed, we can't do this check
	//checkSlow(! (v0->TestFlags(SIMP_LOCKED) && v1->TestFlags(SIMP_LOCKED)) );


    // Verify that this is truly an edge of a triangle

	v0->EnableAdjVertFlags(SIMP_MARK1);
	v1->DisableAdjVertFlags(SIMP_MARK1);

	if (v0->TestFlags(SIMP_MARK1))
	{
		// Invalid edge results from collapsing a bridge tri
		// There are no actual triangles connecting these verts
		v0->DisableAdjVertFlags(SIMP_MARK1);

		EdgePtr->EnableFlags(SIMP_REMOVED);
		const uint32 Idx = RemoveEdge(*EdgePtr);
		if (Idx < INVALID_EDGE_ID)
		{
			RemovedEdgeIdxArray.AddUnique(Idx);
		}

		// return false because the was no real edge to collapse
		return false;
	}

	// update edges from v0 to v1

	// Note, the position and other attributes have already been corrected
	// to have the same values.  Here we are just propagating any locked state.
	if (v0->TestFlags(SIMP_LOCKED))
		v1->EnableFlags(SIMP_LOCKED);

	// this version of the vertex will be removed after the collapse
	v0->DisableFlags(SIMP_LOCKED); // we already shared the locked state with the remaining vertex

	// Update 'other'-u edges to 'other'-v edges ( where other != v) 

	for (TriIterator triIter = v0->adjTris.Begin(); triIter != v0->adjTris.End(); ++triIter)
	{
		SimpTriType* TriPtr = *triIter;
		for (int j = 0; j < 3; j++)
		{
			SimpVertType* VertPtr = TriPtr->verts[j];
			if (VertPtr->TestFlags(SIMP_MARK1))
			{

				// replace v0-vert with v1-vert
				// first remove v1-vert if it already exists ( it shouldn't..)
				{
					uint32 Idx = RemoveEdge(VertPtr, v1);
					if (Idx < INVALID_EDGE_ID)
					{
						RemovedEdgeIdxArray.AddUnique(Idx);
					}

					ReplaceVertInEdge(v0, VertPtr, v1);
				}
				VertPtr->DisableFlags(SIMP_MARK1);
			}
		}
	}

	// For faces with verts: v0, v1, other
	// remove the v0-other edges.
	v0->EnableAdjVertFlags(SIMP_MARK1);
	v0->DisableFlags(SIMP_MARK1);
	v1->DisableFlags(SIMP_MARK1);

	for (TriIterator triIter = v1->adjTris.Begin(); triIter != v1->adjTris.End(); ++triIter)
	{
		SimpTriType* TriPtr = *triIter;
		for (int j = 0; j < 3; j++)
		{
			SimpVertType* VertPtr = TriPtr->verts[j];
			if (VertPtr->TestFlags(SIMP_MARK1))
			{
				const uint32 Idx = RemoveEdge(v0, VertPtr);
				if (Idx < INVALID_EDGE_ID)
				{
					RemovedEdgeIdxArray.AddUnique(Idx);
				}
				//
				VertPtr->DisableFlags(SIMP_MARK1);
			}
		}
	}

	v1->DisableAdjVertFlags(SIMP_MARK1);

	// Remove collapsed triangles, and fix-up the others that now use v instead of u triangles

	TriPtrArray v0AdjTris;
	{
		uint32 i = 0;
		ResizeArray(v0AdjTris, v0->adjTris.Num());


		for (TriIterator triIter = v0->adjTris.Begin(); triIter != v0->adjTris.End(); ++triIter)
		{
			v0AdjTris[i] = *triIter;
			i++;
		}
	}

	for (int32 i = 0, iMax = v0AdjTris.Num(); i < iMax; ++i)
	{
		SimpTriType* TriPtr = v0AdjTris[i];

		checkSlow(!TriPtr->TestFlags(SIMP_REMOVED));
		checkSlow(TriPtr->HasVertex(v0));

		if (TriPtr->HasVertex(v1))  // tri shared by v0 and v1.. 
		{
			// delete triangles on edge uv
			ReducedNumTris--;
			RemoveTri(*TriPtr);
		}
		else
		{
			// update triangles to have v1 instead of v0
			ReplaceTriVertex(*TriPtr, *v0, *v1);
		}
	}


	// remove modified verts and tris from cache
	v1->EnableAdjVertFlags(SIMP_MARK1);
	for (TriIterator triIter = v1->adjTris.Begin(); triIter != v1->adjTris.End(); ++triIter)
	{
		SimpTriType* TriPtr = *triIter;

		for (int i = 0; i < 3; i++)
		{
			SimpVertType* VertPtr = TriPtr->verts[i];
			if (VertPtr->TestFlags(SIMP_MARK1))
			{
				VertPtr->DisableFlags(SIMP_MARK1);
			}
		}
	}

	// mark v0 as dead.

	v0->adjTris.Clear();	// u has been removed
	v0->EnableFlags(SIMP_REMOVED);

	// Remove the actual edge.
	const uint32 Idx = RemoveEdge(*EdgePtr);
	if (Idx < INVALID_EDGE_ID)
	{
		RemovedEdgeIdxArray.AddUnique(Idx);
	}

	// record the reduced number of verts

	ReducedNumVerts--;

	return true;
}

void SkeletalSimplifier::FSimplifierMeshManager::OutputMesh(MeshVertType* verts, uint32* indexes, TArray<int32>* LockedVerts)
{



	int32 NumValidVerts = 0;
	for (int32 i = 0; i < NumSrcVerts; i++)
		NumValidVerts += VertArray[i].TestFlags(SIMP_REMOVED) ? 0 : 1;

	check(NumValidVerts <= ReducedNumVerts);


	FHashTable HashTable(4096, NumValidVerts);
	int32 numV = 0;
	int32 numI = 0;

	for (int32 i = 0; i < NumSrcTris; i++)
	{
		if (TriArray[i].TestFlags(SIMP_REMOVED))
			continue;

		// TODO this is sloppy. There should be no duped verts. Alias by index.
		for (int j = 0; j < 3; j++)
		{
			SimpVertType* vert = TriArray[i].verts[j];
			checkSlow(!vert->TestFlags(SIMP_REMOVED));
			checkSlow(vert->adjTris.Num() != 0);

			const FVector& p = (FVector)vert->GetPos();
			uint32 hash = HashPoint(p);
			uint32 f;
			for (f = HashTable.First(hash); HashTable.IsValid(f); f = HashTable.Next(f))
			{
				if (vert->vert == verts[f])
					break;
			}
			if (!HashTable.IsValid(f))
			{
				// export the id of the locked vert.
				if (LockedVerts != NULL && vert->TestFlags(SIMP_LOCKED))
				{
					LockedVerts->Push(numV);
				}

				HashTable.Add(hash, numV);
				verts[numV] = vert->vert;
				indexes[numI++] = numV;
				numV++;
			}
			else
			{
				indexes[numI++] = f;
			}
		}
	}

#if 0
	check(numV <= NumValidVerts);
	check(numI <= numTris * 3);

	numVerts = numV;
	numTris = numI / 3;
#endif 
}


int32 SkeletalSimplifier::FSimplifierMeshManager::CountDegeneratesTris() const
{
	int32 DegenerateCount = 0;
	// remove degenerate triangles
	// not sure why this happens
	for (int i = 0; i < NumSrcTris; i++)
	{
		const SimpTriType* tri = &TriArray[i];

		if (tri->TestFlags(SIMP_REMOVED))
			continue;

		const FVector& p0 = (FVector)tri->verts[0]->GetPos();
		const FVector& p1 = (FVector)tri->verts[1]->GetPos();
		const FVector& p2 = (FVector)tri->verts[2]->GetPos();
		const FVector n = (p2 - p0) ^ (p1 - p0);

		if (n.SizeSquared() == 0.0f)
		{
			DegenerateCount++;
		}
	}

	return DegenerateCount;
}

int32 SkeletalSimplifier::FSimplifierMeshManager::CountDegenerateEdges() const
{

	int32 DegenerateCount = 0;
	const SkeletalSimplifier::SimpEdgeType*  Edges = EdgeArray.GetData();
	int32 NumEdges = EdgeArray.Num();

	for (int32 i = 0; i < NumEdges; ++i)
	{
		const SkeletalSimplifier::SimpEdgeType& Edge = Edges[i];

		if (Edge.TestFlags(SIMP_REMOVED))
			continue;

		if (Edge.v0 == Edge.v1)
		{
			DegenerateCount++;
		}

	}

	return DegenerateCount;
}

#if 0
// disabled because static analysis insists that NonManifoldEdgeCounter.EdgeCount != 0 is always false.
float  SkeletalSimplifier::FSimplifierMeshManager::FractionNonManifoldEdges(bool bLockNonManifoldEdges)
{

	FNonManifoldEdgeCounter NonManifoldEdgeCounter;
	NonManifoldEdgeCounter.EdgeCount = 0;
	NonManifoldEdgeCounter.NumNonManifoldEdges = 0;
	NonManifoldEdgeCounter.bLockNonManifoldEdges = bLockNonManifoldEdges;


	VisitEdges(NonManifoldEdgeCounter);

	float FractionBadEdges =  0.f;
	if (NonManifoldEdgeCounter.EdgeCount != 0)
	{ 
		FractionBadEdges = float(NonManifoldEdgeCounter.NumNonManifoldEdges) / NonManifoldEdgeCounter.EdgeCount;
	}

	return FractionBadEdges;
}
#else

float SkeletalSimplifier::FSimplifierMeshManager::FractionNonManifoldEdges(bool bLockNonManifoldEdges)
{

	int32 NumVisitedEdges = 0;
	int32 NumNonManifoldEdges = 0;

	TArray< SimpVertType*, TInlineAllocator<64> > adjVerts;
	if (NumSrcVerts == 0 || NumSrcTris == 0)
	{
		//Avoid trying to compute an empty mesh
		return 0.f;
	}

	// clear the mark2 flags. We use these to determine if we have visited a vert group.
	for (int32 i = 0; i < NumSrcVerts; ++i)
	{
		SimpVertType* v0 = &VertArray[i];
		v0->DisableFlags(SIMP_MARK2);
	}

	for (int i = 0; i < NumSrcVerts; i++)
	{

		SimpVertType* v0 = &VertArray[i];
		checkSlow(v0 != NULL);
		check(v0->adjTris.Num() > 0);

		// we have already visited this vertex group
		if (v0->TestFlags(SIMP_MARK2))
		{
			continue;
		}

		if (v0->TestFlags(SIMP_REMOVED))
		{
			continue;
		}

		//Find all the verts that are adjacent to any vert in this group.
		adjVerts.Reset();
		SimpVertType* v0Smallest = v0;
		{
			SimpVertType* v = v0;
			do {
				for (TriIterator triIter = v->adjTris.Begin(); triIter != v->adjTris.End(); ++triIter)
				{
					for (int j = 0; j < 3; j++)
					{
						SimpVertType* TriVert = (*triIter)->verts[j];
						if (TriVert != v)
						{
							adjVerts.AddUnique(TriVert);
						}
					}
				}
				v = v->next;
				if (v0Smallest > v)
				{
					v0Smallest = v;
				}
			} while (v != v0);
		}

		for (SimpVertType* v1 : adjVerts)
		{
			// visit edges that are incoming to this vertex group
			// note, we may end up visiting a few edges twice.
			if (v0Smallest < v1)
			{

				// set if this edge is boundary
				// find faces that share v0 and v1
				v0->EnableAdjTriFlagsGroup(SIMP_MARK1);
				v1->DisableAdjTriFlagsGroup(SIMP_MARK1);

				int32 AdjFaceCount = 0;
				SimpVertType* vert = v0;
				do
				{
					for (TriIterator j = vert->adjTris.Begin(); j != vert->adjTris.End(); ++j)
					{
						SimpTriType* tri = *j;
						AdjFaceCount += tri->TestFlags(SIMP_MARK1) ? 0 : 1;
					}
					vert = vert->next;
				} while (vert != v0);

				// reset v0-group flag.
				v0->DisableAdjTriFlagsGroup(SIMP_MARK1);

				// process this edge.
				{
					NumVisitedEdges++;
					if (AdjFaceCount > 2)
					{
						NumNonManifoldEdges++;
						if (bLockNonManifoldEdges)
						{
							// lock these verts.
							v0->EnableFlagsGroup(SIMP_LOCKED);
							v1->EnableFlagsGroup(SIMP_LOCKED);
						}
					}
				}
			}
		}

		// visited this vert and all the incoming edges.
		v0->EnableFlagsGroup(SIMP_MARK2);
	}

	for (int32 i = 0; i < NumSrcVerts; ++i)
	{
		SimpVertType* v0 = &VertArray[i];
		v0->DisableFlags(SIMP_MARK2);
	}

	return float(NumNonManifoldEdges) / (float(NumVisitedEdges) +0.01f);
}

#endif
