// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once


#include "MeshSimplifyElements.h"

#include "MeshUVChannelInfo.h"
#include "SkeletalSimplifierVertex.h"


namespace SkeletalSimplifier
{


	template <typename ValueType, typename AllocatorType>
	void ResizeArray(TArray<ValueType, AllocatorType>& Array, const int32 Size)
	{
		Array.Empty(Size);
		Array.AddUninitialized(Size);
	}

	/**
	*  Define the Simplifier types needed.
	*/
 
	typedef VertexTypes::TBasicVertexAttrs<MAX_TEXCOORDS>	BasicAttrArray;
	typedef VertexTypes::DynamicVertexAttrs                 SparseAttrArray;
	typedef VertexTypes::BoneSparseVertexAttrs              BoneAttrArray;

	typedef VertexTypes::TSkeletalSimpVert<BasicAttrArray, SparseAttrArray, BoneAttrArray>    MeshVertType;

	// Specialized Link-Lists for Verts and Edges
	typedef TSimpVert< MeshVertType >    SimpVertType;
	typedef TSimpEdge< MeshVertType >    SimpEdgeType;

	// Definition of a triangle.
	typedef TSimpTri< MeshVertType >     SimpTriType;

	/**
	* Mesh type internal to the simplifier.  This is really an attempt to tease the mesh management
	* code out of the simplifier.  Not intended to be a general mesh.
	* This mesh may be updated by collapsing edges, altering attributes.  In doing so, the mesh
	* does not delete any vertices, just marks them as removed.
	*
	* All attributes are stored in the vertices.
	* Vertices are assumed to be split, so that multiple vertices may coincide at the same physical location.
	* Likewise with edges.
	*
	* As an organizational principle, coincident vertices are held in pointer-based link lists, as are edges.
	* These are often referenced as vertex groups or edge groups, but in practice a group can be infered from
	* a single member. 
	*
	 
	*/
	class FSimplifierMeshManager
	{
	public:
		typedef TArray< SimpEdgeType*, TInlineAllocator<32> >                 EdgePtrArray;
		typedef TArray< SimpTriType*, TInlineAllocator<16> >                  TriPtrArray;
		typedef TArray< SimpVertType*, TInlineAllocator<16> >                 VertPtrArray;
		typedef TArray< uint32, TInlineAllocator<8> >                         IdxArray;

		typedef typename SimpVertType::TriIterator	                         TriIterator;

		typedef TSharedPtr<FSimplifierMeshManager>                                   Ptr;

		FSimplifierMeshManager(const MeshVertType* InSrcVerts, const uint32 InNumSrcVerts,
			const uint32* InSrcIndexes, const uint32 InNumSrcIndexes, const bool bMergeBonesIfSamePos);

		~FSimplifierMeshManager() = default;

		// Extract the currently valid verts / indices from this object.  If LockedVerts != NULL
		// then the indices of verts that had locked state will be passed out too.
		void OutputMesh(MeshVertType* verts, uint32* indexes, TArray<int32>* LockedVerts = NULL);



		// Apply the flag to all verts on the edge of the mesh.
		void        FlagBoundary(const ESimpElementFlags Flag);

		// Apply a flag to all verts that are identified as being at the corner of a box.
		void        FlagBoxCorners(const ESimpElementFlags Flag);

		// Apply flag to edges when the IsDifferent(AVert, BVert) == true
		void        FlagEdges(const TFunction<bool(const SimpVertType*, const SimpVertType*)> IsDifferent, const ESimpElementFlags Flag);

		// Visit each edge group, and call EdgeVisitor(VertexA, VertexB, NumAdjacentFaces)
		void        VisitEdges(TFunctionRef<void(SimpVertType*, SimpVertType*, int32)> EdgeVisitor);

		// Change the attributes on a given simplifier vert.
		void UpdateVertexAttributes(SimpVertType& Vertex, const MeshVertType& AttributeVert)
		{

			Vertex.vert = AttributeVert;
		}

		// copy the element IDs for the correct attributes from v1 to v0 in preparation for collapse.
		void       UpdateVertexAttriuteIDs(EdgePtrArray& CoincidentEdges);

		// Count the number of triangles with zero area.
		int32 CountDegeneratesTris() const;

		// Count the number of edges with zero length.
		int32 CountDegenerateEdges() const;

		// Fraction (0 to 1) of edge-groups with more than two adjacent tris, optionally lock these edges
		float FractionNonManifoldEdges(bool bLockNonManifoldEdges = false);

		// Hash location 
		static uint32 HashPoint(const FVector& p)
		{
			union { float f; uint32 i; } x;
			union { float f; uint32 i; } y;
			union { float f; uint32 i; } z;

			x.f = p.X;
			y.f = p.Y;
			z.f = p.Z;

			return Murmur32({ x.i, y.i, z.i });
		}

		int32 TotalNumEdges() const
		{
			return EdgeArray.Num();
		}

		// Return true if either vertex has no associated faces.
		bool IsInvalid(const SimpEdgeType* EdgePtr) const
		{
			 return (EdgePtr->v0->adjTris.Num() == 0 || EdgePtr->v1->adjTris.Num() == 0);
		}

		// Return true if the AVert and BVert have the same vertex locations.  Optionally require the same bones.
		bool IsCoincident(const SimpVertType* AVertPtr, const SimpVertType* BVertPtr, bool bCompairBones = true) const
		{
			bool bCoincident = ( AVertPtr->GetPos() == BVertPtr->GetPos()  && (!bCompairBones || AVertPtr->vert.GetSparseBones().IsApproxEquals(BVertPtr->vert.GetSparseBones())) );
			return bCoincident;
		}

		bool IsRemoved(const SimpEdgeType* EdgePtr) const
		{
			return EdgePtr->TestFlags(SIMP_REMOVED);
		}

		SimpEdgeType* GetEdgePtr(const uint32 Idx)
		{
			checkSlow(Idx < uint32(EdgeArray.Num()));
			return &EdgeArray[Idx];
		}

		const SimpEdgeType* GetEdgePtr(const uint32 Idx) const
		{
			checkSlow(Idx < uint32(EdgeArray.Num()));
			return &EdgeArray[Idx];
		}

		SimpVertType* GetVertPtr(const uint32 Idx)
		{
			checkSlow(Idx < uint32(NumSrcVerts));
			return &VertArray[Idx];
		}

		const SimpVertType* GetVertPtr(const uint32 Idx) const
		{
			checkSlow(Idx < uint32(NumSrcVerts));
			return &VertArray[Idx];
		}


		// Convert pointer to array index 
		uint32 GetVertIndex(const SimpVertType* VertPtr) const
		{
			ptrdiff_t Index = VertPtr - &VertArray[0];
			return (uint32)Index;
		}
		// Convert pointer to array index
		uint32 GetTriIndex(const SimpTriType* TriPtr) const
		{
			ptrdiff_t Index = TriPtr - &TriArray[0];
			return (uint32)Index;
		}
		// Convert pointer to array index
		uint32 GetEdgeIndex(const SimpEdgeType* EdgePtr) const
		{
			ptrdiff_t Index = EdgePtr - &EdgeArray[0];
			return (uint32)Index;
		}


		// Merge two link-lits of verts into a single group.
		// NB: this does not verify that the locations of the verts are all the same.

		void MergeGroups(SimpVertType*  A, SimpVertType*  B)
		{
			// This inserts the B-loop between A and A.next.

			// combine v0 and v1 groups
			A->next->prev = B->prev;
			B->prev->next = A->next;

			A->next = B;
			B->prev = A;
			
			/**
			// This inserts the B-loop between A.prev and A.
			// the last B in the B-loop
			SimpVertType* Blast = B->prev;
			// the last A in the A-Loop
			SimpVertType* Alast = A->prev;

			// Add A-loop after the last B
			Blast->next = A;
			A->prev = Blast;

			// make the last 
			B->prev = Alast;
			Alast->next = B;
			*/
		}

		// If a member of the group has the given flag, propagate that flag
		// to each member
		template < ESimpElementFlags FlagToPropagate >
		void PropagateFlag(SimpVertType& MemberOfGroup)
		{
			// spread locked flag to vert group
			uint32 flags = 0;

			SimpVertType* v = &MemberOfGroup;
			do {
				flags |= v->flags & FlagToPropagate;
				v = v->next;
			} while (v != &MemberOfGroup);

			v = &MemberOfGroup;
			do {
				v->flags |= flags;
				v = v->next;
			} while (v != &MemberOfGroup);
		}

		//  Gather all the edges implicitly in the edge group defined by the seed edge
		void GetEdgesInGroup(const SimpEdgeType& seedEdge, EdgePtrArray& InOutEdgeGroup) const
		{
			SimpEdgeType* EdgePtr = const_cast<SimpEdgeType*>(&seedEdge);
			do {
				InOutEdgeGroup.Add(EdgePtr);
				EdgePtr = EdgePtr->next;
			} while (EdgePtr != &seedEdge);
		}

		//  Gather all the verts in the group defined by this vert.
		void GetVertsInGroup(const SimpVertType& seedVert, VertPtrArray& InOutVertGroup) const
		{
			SimpVertType* VertPtr = const_cast<SimpVertType*>(&seedVert);
			SimpVertType* v0 = VertPtr;
			do {
				InOutVertGroup.Add(v0);
				v0 = v0->next;
			} while (v0 != VertPtr);
		}

		// Remove any verts tagged as with FlagValue (e.g. SIMP_REMOVED)  NB: this may unlink the seed vert!
		template < ESimpElementFlags FlagValue >
		void PruneVerts(SimpVertType& seedVert)
		{
			// Get all the verts that are link-listed together
			VertPtrArray VertsInVertGroup;
			GetVertsInGroup(seedVert, VertsInVertGroup);

			// Unlink any verts that are marked as SIMP_REMOVED
			for (int32 i = 0, Imax = VertsInVertGroup.Num(); i < Imax; ++i)
			{
				SimpVertType* v = VertsInVertGroup[i];
				if (v->TestFlags(FlagValue))
				{
					// ungroup
					v->prev->next = v->next;
					v->next->prev = v->prev;
					v->next = v;
					v->prev = v;

					// cleanup.  Insure that pruned verts aren't marked as locekd.
					v->DisableFlags(SIMP_LOCKED);
				}
			}

		}

		// Count the adjacent tris to all the split verts that make up this vert.
		uint32 GetDegree(const SimpVertType& Vert) const
		{
			uint32 Degree = 0;

			const SimpVertType* StartVertPtr = &Vert;

			const SimpVertType* VertPtr = StartVertPtr;
			do {
				Degree += VertPtr->adjTris.Num();
				VertPtr = VertPtr->next;
			} while (VertPtr != StartVertPtr);


			return Degree;
		}
		// Gather all the tri, verts, and edges that will be affected by moving this vert.
		void GetAdjacentTopology(const SimpVertType* VertPtr,
			TriPtrArray& AdjacentTris, VertPtrArray& AdjacentVerts, EdgePtrArray& AdjacentEdges);

		void GetAdjacentTopology(const SimpEdgeType& GroupedEdge,
			TriPtrArray& AdjacentTris, VertPtrArray& AdjacentVerts, EdgePtrArray& AdjacentEdges);


		// Get the an array of vert groups, in the form of pointers to the first element in each group.
		// The verts in each group share the same position.

		void GetCoincidentVertGroups(VertPtrArray& CoincidentVertGroups);

		// Weld non-split basic attributes on coincident vertices. This should be called prior to output.
	    // Note: The simplifier will split a vertex into multiple attribute vertices (with a full copy of all attributes) if only one attribute is split,
	    // this insures that the non-split attributes share the same value.
		enum class EVtxElementWeld
		{
			Normal,
			Tangent,
			BiTangent,
			Color,
			UV,
		};
		void WeldNonSplitBasicAttributes(EVtxElementWeld WeldType);

		// note, this shares a lot of code with GroupEdges - they should be unified..
		void RebuildEdgeLinkLists(EdgePtrArray& CandidateEdgePtrArray);



		// On return CandidateEdges[i] = NULL for any removed edge and the 
		// Edge Idx of all the removed edges are stored in the RemovedEdgeIdArray
		int32 RemoveEdgeIfInvalid(EdgePtrArray& CandidateEdges, IdxArray& RemovedEdgeIdxArray);

		// @return the idx of this edge it if is currently in the VertIdHashMap
		// IF the edge has already been deleted, or doesn't exist, return numeric_limits<uint32>::max()
		uint32 RemoveEdge(const SimpVertType* VertAPtr, const SimpVertType* VertBPtr);

		// @return the idx of the edge after removal to allow the caller to clean up any associated data structures (e.g. heap)
		uint32 RemoveEdge(SimpEdgeType& Edge);


		// Change the edge VertA-VertB to  VertAprime-VertB
		// @return the Idx of the edge that was changed.
		uint32 ReplaceVertInEdge(const SimpVertType* VertAPtr, const SimpVertType* VertBPtr, SimpVertType* VertAprimePtr);


		// @return the number of removed tris.
		// Flags tris as SIMP_REMOVED and removes them from the verts that referenced them
		int32 RemoveIfDegenerate(TriPtrArray& CandidateTrisPtrArray);

		int32 RemoveDegenerateTris();

		// Collapse the edge by moving edge-v0 to edge->v1. and record the 
		// idx of the edges that are deleted by this action. 
		// Note, collapsing a single edge may result in additional edges being
		// removed.  A triangle that shares this edge will be reduced
		// to a single edge
		// e.g. collapse edge 0-1 in the triangle 0-1,  1-2, 2-0
		//      will result in the single edge 1-2
		// @returns true if the edge collapsed, false if the edge wasn't a true edge.
		bool CollapseEdge(SimpEdgeType* EdgePtr, IdxArray& RemovedEdgeIdxArray);

		// Mark a tri as removed, and remove it from vertex adj lists.
		uint32 RemoveTri(SimpTriType& Tri)
		{
			Tri.EnableFlags(SIMP_REMOVED);

			// remove the tri from all verts
			for (int j = 0; j < 3; ++j)
			{
				SimpVertType* V = Tri.verts[j];
				V->adjTris.Remove(&Tri);
			}

			return GetTriIndex(&Tri);
		}

		// On @return the index of the tri
		uint32 ReplaceTriVertex(SimpTriType& Tri, SimpVertType& OldVert, SimpVertType& NewVert)
		{
			Tri.ReplaceVertex(&OldVert, &NewVert);
			NewVert.adjTris.Add(&Tri);
			OldVert.adjTris.Remove(&Tri);

			return GetTriIndex(&Tri);
		}
		// @return the number of removed verts
		// Flags the removed verts link elements as SIMP_REMOVED and removes verts from vert link lists.
		int32 RemoveIfDegenerate(VertPtrArray& CandidateVertPtrArray);

		int32 RemoveDegenerateVerts();


		// On return CandidateEdges[i] = NULL for any removed edge and the 
		// Edge Idx of all the removed edges are stored in the RemovedEdgeIdArray
		int32 RemoveIfDegenerate(EdgePtrArray& CandidateEdges, IdxArray& RemovedEdgeIdxArray);


		// @return true if any of the edges in this edge group is locked.
		bool IsLockedGroup(const EdgePtrArray& EdgeGroup) const
		{
			bool locked = false;
			int32 NumEdgesInGroup = EdgeGroup.Num();
			for (int32 i = 0; i < NumEdgesInGroup; ++i)
			{
				const SimpEdgeType* edge = EdgeGroup[i];

				if (edge->v0->TestFlags(SIMP_LOCKED) && edge->v1->TestFlags(SIMP_LOCKED))
				{
					locked = true;
					break;
				}
			}
			return locked;
		}

		//@return true if the either of the verts are locked
		bool HasLockedVerts(const SimpEdgeType* Edge) const
		{
			return (Edge->v0->TestFlags(SIMP_LOCKED) || Edge->v1->TestFlags(SIMP_LOCKED));
		}


		// Find the edge associated with these verts.  Will return NULL
		// if no such edge exists.
		SimpEdgeType* FindEdge(const SimpVertType* u, const SimpVertType* v)
		{
			uint32 idx = GetEdgeHashPair(u, v).Key;

			return (idx < UINT32_MAX ) ? &EdgeArray[idx] : NULL;

		}

		// Weld Bones if the vertices have the same location
		bool bWeldBonesIfSamePos = true;

		// The number of verts and tris in the initial mesh.
		const int32					NumSrcVerts = 0;
		const int32					NumSrcTris = 0;

		int32   ReducedNumVerts;
		int32   ReducedNumTris;

		// Note after these arrays are constructed, they should never be resized.
		// code holds pointers to array elements.
		TArray<SimpVertType>  VertArray;
		TArray<SimpTriType>   TriArray;


		// Hash based on the Ids of the edge's verts.
		// used to map verts to edges.

		FHashTable				EdgeVertIdHashMap;

		// Array of edges
		// Note - this array shouldn't be resized after the mesh is constructed
		TArray< SimpEdgeType >	EdgeArray;

	private:

		// Methods used in the initial construction of the simplifier mesh

		void GroupVerts(TArray<SimpVertType>& Verts);
		void SetAttributeIDS(TArray<SimpVertType>& Verts);
		void MakeEdges(const TArray<SimpVertType>& Verts, const int32 NumTris, TArray<SimpEdgeType>& Edges);
		void AppendConnectedEdges(const SimpVertType* Vert, TArray<SimpEdgeType>& Edges);
		void GroupEdges(TArray< SimpEdgeType >& Edges);


		// Generate a hash based on the Idxs of the verts.
		// Note this is independent of the order of the verts.
		uint32 HashEdge(const SimpVertType* u, const SimpVertType* v) const
		{
			uint32 ui = GetVertIndex(u);
			uint32 vi = GetVertIndex(v);
			// must be symmetrical
			return Murmur32({ FMath::Min(ui, vi), FMath::Max(ui, vi) });
		}

		uint32 HashEdgePosition(const SimpEdgeType& Edge)
		{
			return HashPoint((FVector)Edge.v0->GetPos()) ^ HashPoint((FVector)Edge.v1->GetPos());
		}

		TPair<uint32, uint32> GetEdgeHashPair(const SimpVertType* u, const SimpVertType* v) const
		{
			uint32 hashValue = HashEdge(u, v);

			TPair<uint32, uint32> Result(UINT32_MAX, hashValue);
			for (uint32 i = EdgeVertIdHashMap.First(hashValue); EdgeVertIdHashMap.IsValid(i); i = EdgeVertIdHashMap.Next(i))
			{
				if ((EdgeArray[i].v0 == u && EdgeArray[i].v1 == v) ||
					(EdgeArray[i].v0 == v && EdgeArray[i].v1 == u))
				{
					Result.Key = i;
					break;
				}
			}

			return Result;
		}

		// struct used in sorting and merging coincident verts.
		struct FVertAndID
		{
			int32 ID;
			SimpVertType* SrcVert;

			FVertAndID() {};
			FVertAndID(SimpVertType* SV, int32 InID)
			{
				ID = InID;
				SrcVert = SV;
			}
		};

		// struct used with VistEdges when counting nonmanifold edges.
		struct FNonManifoldEdgeCounter
		{
			int32 EdgeCount;
			int32 NumNonManifoldEdges;
			bool bLockNonManifoldEdges;

			void operator()(SimpVertType* v0, SimpVertType* v1, int32 AdjFaceCount)
			{
				EdgeCount++;
				if (AdjFaceCount > 2)
				{
					NumNonManifoldEdges++;

					if (bLockNonManifoldEdges)
					{
						// lock these verts.
						v0->EnableFlagsGroup(SIMP_LOCKED);
						v1->EnableFlagsGroup(SIMP_LOCKED);
					}
				}
			}

		};

	};
}
