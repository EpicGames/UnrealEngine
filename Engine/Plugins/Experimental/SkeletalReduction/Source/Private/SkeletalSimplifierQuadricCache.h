// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "SkeletalSimplifierMeshManager.h"
#include "SkeletalSimplifierQuadrics.h"

namespace SkeletalSimplifier
{

	/**
	* Cache to manage quadrics for the quadric mesh reduction.
	*
	*/
	template <typename WedgeQuadricType>
	class TQuadricCache
	{
	public:
		typedef Quadrics::FEdgeQuadric  EdgeQuadricType;

		TQuadricCache() {}

		/**
		* Associate the cache with the simplifier mesh
		* This must be done before the cache can be used.
		*
		* @param Mesh  The FSimplifierMesh that this cache will be associated with
		*/
		void RegisterMesh(const FSimplifierMeshManager& Mesh)
		{
			RegisterCache(Mesh.VertArray, Mesh.TriArray);
		}

		/**
		* Get the Quadric.
		* If the quadric for the requested SimplifierVert is out of date (dirty) then use
		* the supplied TriQuadricFactor functor to compute a new quadric (and add it to the cache)
		*
		* QuadricType  TriQuadricFactor(SimpTriType& Tri);
		*
		* @param v                  The vert
		* @param TriQuadricFactor   Method to compute new quadric from a triangle
		*/
		template <typename TriQuadricFatoryType>
		WedgeQuadricType GetWedgeQuadric(SimpVertType* v, const TriQuadricFatoryType& TriQuadricFactory);


		/**
		* Get the Edge Quadric.
		* If the edge quadric for the requested SimplifierVert is out of date (dirty) then use
		* the supplied EdgeQuadricFatory functor to compute a new quadric (and add it to the cache)
		*
		* FQuadric  EdgeQuadricFatory(v->GetPos(), vert->GetPos(), face->GetNormal());
		*
		* @param v                  The vert
		* @param EdgeQuadricFatory   Method to compute new quadric from a triangle
		*/
		template <typename EdgeQuadricFatoryType>
		EdgeQuadricType GetEdgeQuadric(SimpVertType* v, const EdgeQuadricFatoryType& EdgeQuadricFatory);


		/**
		* Mark the associated vertex quadric as dirty in the cache.
		*/
		void DirtyVertQuadric(const SimpVertType* v);
		void DirtyVertQuadric(const uint32 VertIdx);

		/**
		* Mark the associated triangle quadric as dirty in the cache.
		*/
		void DirtyTriQuadric(const SimpTriType* tri);
		void DirtyTriQuadric(const uint32 TridIdx);

		/**
		* Mark the associated edge quadric as dirty in the cache.
		*/
		void DirtyEdgeQuadric(const SimpVertType* v);
		void DirtyEdgeQuadric(const uint32 VertIdx);


	private:
		uint32 GetVertIndex(const SimpVertType* vert) const;

		uint32 GetTriIndex(const SimpTriType* tri) const;


		/**
		* Associate the cache with the simplifier vert and triangle arrays.
		* This must be done before the cache can be used.
		*
		* @param VertArray   Array of verts in the source mesh
		* @param TriArray    Array of Tris in the source mesh
		*/
		void RegisterCache(const TArray<SimpVertType>& VertArray, const TArray<SimpTriType>& TriArray);

		/**
		* Allocate the member arrays with the size required
		* This must be done before the cache can be used.
		*
		* @param NumVerts    Number of Verts in the source mesh
		* @param NumTris     Number of tris in the source mesh
		*/
		void AllocateCacheArrays(const int32 NumVerts, const int32 NumTris);

	private:
		// Disable

		TQuadricCache(const TQuadricCache& other);

	private:

		TBitArray<>				        VertQuadricsValid;
		TArray< WedgeQuadricType >	    VertQuadrics;

		TBitArray<>				        TriQuadricsValid;
		TArray< WedgeQuadricType >	    TriQuadrics;

		TBitArray<>				        EdgeQuadricsValid;
		TArray< EdgeQuadricType >		EdgeQuadrics;

		// To map vert pointer to vert index

		const SimpVertType*		sVerts = NULL;
		const SimpTriType*		sTris = NULL;
	};




	//=============
	// TQuadricCache
	//=============
	template <typename WedgeQuadricType >
	void TQuadricCache< WedgeQuadricType >::AllocateCacheArrays(const int32 NumSVerts, const int32 NumSTris)
	{
		VertQuadricsValid.Init(false, NumSVerts);
		VertQuadrics.SetNum(NumSVerts);

		TriQuadricsValid.Init(false, NumSTris);
		TriQuadrics.SetNum(NumSTris);

		EdgeQuadricsValid.Init(false, NumSVerts);
		EdgeQuadrics.SetNum(NumSVerts);
	}



	template <typename WedgeQuadricType>
	void TQuadricCache< WedgeQuadricType>::RegisterCache(const TArray<SimpVertType>& VertArray,  const TArray<SimpTriType>& TriArray )
	{
		const int32 NumVerts = VertArray.Num();
		const int32 NumTris  = TriArray.Num();
	
		sVerts = VertArray.GetData();
		sTris  = TriArray.GetData();

		AllocateCacheArrays(NumVerts, NumTris);
	}

	// Get the quadric 


	template <typename WedgeQuadricType>
	template <typename TriQuadricFatoryType>
	WedgeQuadricType TQuadricCache< WedgeQuadricType>::GetWedgeQuadric(SimpVertType* v, const TriQuadricFatoryType& TriQuadricFactory)
	{
		// checks to verify the cache has been registered
		checkSlow(sVerts != NULL);
		checkSlow(sTris != NULL);

		uint32 VertIndex = GetVertIndex(v);
		if (VertQuadricsValid[VertIndex])
		{
			return VertQuadrics[VertIndex];
		}

		static int32 NumBasicAttrs = WedgeQuadricType::SimpVertexType::NumBaseAttributes();
		WedgeQuadricType vertQuadric;
		// @Alert!
		// djh - do I need this?? vertQuadric.Zero();

		// sum tri quadrics
		for (auto i = v->adjTris.Begin(); i != v->adjTris.End(); ++i)
		{
			SimpTriType* tri = *i;
			uint32 TriIndex = GetTriIndex(tri);
			if (TriQuadricsValid[TriIndex])
			{
				vertQuadric += TriQuadrics[TriIndex];
			}
			else
			{
				WedgeQuadricType triQuadric = TriQuadricFactory(*tri);
				vertQuadric += triQuadric;

				TriQuadricsValid[TriIndex] = true;
				TriQuadrics[TriIndex] = triQuadric;
			}
		}

		VertQuadricsValid[VertIndex] = true;
		VertQuadrics[VertIndex] = vertQuadric;

		return vertQuadric;
	}



	template <typename WedgeQuadricType>
	template <typename EdgeQuadricFatoryType>
	Quadrics::FEdgeQuadric TQuadricCache< WedgeQuadricType>::GetEdgeQuadric(SimpVertType* v, const EdgeQuadricFatoryType& EdgeQuadricFatory)
	{
		// checks to verify the cache has been registered
		checkSlow(sVerts != NULL);
		checkSlow(sTris != NULL);

		uint32 VertIndex = GetVertIndex(v);
		if (EdgeQuadricsValid[VertIndex])
		{
			return EdgeQuadrics[VertIndex];
		}

		Quadrics::FEdgeQuadric edgeQuadric;
		//@alert djh - This should be constructed with zero value
		edgeQuadric.Zero();

		TArray< SimpVertType*, TInlineAllocator<64> > adjVerts;
		v->FindAdjacentVerts(adjVerts);

		// djh 
		//	LockTriFlags(SIMP_MARK1);
		v->EnableAdjTriFlags(SIMP_MARK1);

		// Goal : only add non-zero edge quadric if the this edge has only one face adjacent.
		// Why? Because a 1 face edge may just be a discontinuity at the UVs and not a real edge of 
		// the mesh.

		for (SimpVertType* vert : adjVerts)
		{
			SimpTriType* face = NULL;
			int faceCount = 0;
			for (auto j = vert->adjTris.Begin(); j != vert->adjTris.End(); ++j)
			{
				SimpTriType* tri = *j;
				if (tri->TestFlags(SIMP_MARK1))
				{
					face = tri;
					faceCount++;
				}
			}
			
			if (faceCount == 1)
			{
				// only one face on this edge.  Note, GetNormal attempts to return a unit normal, but this can fail
				edgeQuadric += EdgeQuadricFatory((FVector)v->GetPos(), (FVector)vert->GetPos(), (FVector)face->GetNormal());
			}
		}


		v->DisableAdjTriFlags(SIMP_MARK1);
		//	UnlockTriFlags(SIMP_MARK1);

		EdgeQuadricsValid[VertIndex] = true;
		EdgeQuadrics[VertIndex] = edgeQuadric;

		return edgeQuadric;
	}

	template <typename WedgeQuadricType>
	FORCEINLINE void TQuadricCache< WedgeQuadricType>::DirtyVertQuadric(const uint32 VertIdx)
	{
		VertQuadricsValid[VertIdx] = false;
	}

	template <typename WedgeQuadricType>
	FORCEINLINE void TQuadricCache< WedgeQuadricType>::DirtyVertQuadric(const SimpVertType* v)
	{
		DirtyVertQuadric(GetVertIndex(v));
	}

	template <typename WedgeQuadricType>
	FORCEINLINE void TQuadricCache< WedgeQuadricType>::DirtyTriQuadric(const uint32 TriIdx)
	{
		TriQuadricsValid[TriIdx] = false;
	}

	template <typename WedgeQuadricType>
	FORCEINLINE void TQuadricCache< WedgeQuadricType>::DirtyTriQuadric(const SimpTriType* tri)
	{
		DirtyTriQuadric(GetTriIndex(tri));
	}

	template <typename WedgeQuadricType>
	FORCEINLINE void TQuadricCache< WedgeQuadricType>::DirtyEdgeQuadric(const uint32 VertIdx)
	{
		EdgeQuadricsValid[VertIdx] = false;
	}

	template <typename WedgeQuadricType>
	FORCEINLINE void TQuadricCache< WedgeQuadricType>::DirtyEdgeQuadric(const SimpVertType* v)
	{
		DirtyEdgeQuadric(GetVertIndex(v));
	}

	template <typename WedgeQuadricType>
	FORCEINLINE uint32 TQuadricCache< WedgeQuadricType>::GetVertIndex(const SimpVertType* vert) const
	{
		ptrdiff_t Index = vert - &sVerts[0];
		return (uint32)Index;
	}

	template <typename WedgeQuadricType>
	FORCEINLINE uint32 TQuadricCache< WedgeQuadricType>::GetTriIndex(const SimpTriType* tri) const
	{
		ptrdiff_t Index = tri - &sTris[0];
		return (uint32)Index;
	}
}
