// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once


#include "SkeletalSimplifierLinearAlgebra.h"


namespace SkeletalSimplifier
{
	namespace Quadrics
	{

		// For the vectors, matrices and such
		using namespace SkeletalSimplifier::LinearAlgebra;


		// Quadric to preserve the geometry of discontinuity 

		class FEdgeQuadric
		{
		public:

			FEdgeQuadric()
			{}

			FEdgeQuadric(const Vec3d& Vert0Pos, const Vec3d& Vert1Pos, const Vec3d& FaceNormal, const double EdgeWeight);

			FEdgeQuadric(const FEdgeQuadric&) = default;
			FEdgeQuadric& operator=(const FEdgeQuadric&) = default;
			
			double Evaluate(const Vec3d& Pos) const;
			

			FEdgeQuadric& operator+=(const FEdgeQuadric& Other);
		
			void Zero()
			{
				CMatrix.Zero();
				D0Vector.Zero();

				CScalar = 0.;
			}

		private:
			// NB: these all default construct with zero value
			// This contains the minimal data required for the geometric bits of a quadric
			/**
			*                        3
			* Quadric Matrix   (  C_Matrix )   3
			*
			* Quadric Vector   (D0_Vector)  3

			* Quadric Scalar    CScalar     size 1
			*/

			// quadric matrix
			SymmetricMatrix  CMatrix;   // 3x3  - Symmetric matrix
			
			// Quadratic vector							
			Vec3d            D0Vector;  // 3 - vector
		    
		    // Quadric scalar
			double          CScalar = 0.; // 1 - scalar

			// Allow the optimizer access to the member data.  This is used in adding FEdgeQuadric to the Optimizer
			template <typename T>
			friend class TQuadricOptimizer;

		};

		// Base class for the more general Quadric that includes two arrays of attributes.
		// The first array is dense, and the second array is optionally sparse.
		template <typename BasicAttrContainerType, typename AdditionalAttrContainerType>
		class TQuadricBase
		{
		public:

			typedef typename BasicAttrContainerType::DenseBMatrixType                          B1MatrixType;
			typedef typename TVectorToMatrixTrait<AdditionalAttrContainerType>::BMatrixType    B2MatrixType;
			typedef Vec3d                                                                      D0VectorType;
			typedef typename BasicAttrContainerType::DenseVecDType                             D1VectorType;
			typedef AdditionalAttrContainerType                                                D2VectorType;
		

			TQuadricBase() {}


			TQuadricBase(const TQuadricBase& Other) :
				CMatrix(Other.CMatrix),
				B1Matrix(Other.B1Matrix),
				B2Matrix(Other.B2Matrix),
				Gamma(Other.Gamma),
				D0Vector(Other.D0Vector),
				D1Vector(Other.D1Vector),
				D2Vector(Other.D2Vector),
				CScalar(Other.CScalar),
				VolDistConstraint(Other.VolDistConstraint),
				VolGradConstraint(Other.VolGradConstraint),
				UVBBox(Other.UVBBox)
			{}


			TQuadricBase& operator+=(const TQuadricBase& Other);

			TQuadricBase& operator=(const TQuadricBase& Other);

			/**
			* The state vector is broken into three parts:  State = {Pos, S1, S2}
			*
			* This returns double = < State, Quadic_Matrix * State> + 2 <State, Quadric_Vector> + Quadric_Scalar
			*/
			double EvaluateQaudric(const D0VectorType& Pos, const D1VectorType& S1, const D2VectorType& S2) const;

		protected:

			// NB: these all default construct with zero value
			/**
			*                        3               N           M
			* Quadric Matrix   (  C_Matrix,      B1_Matrix,  B2_Matrix )   3
			*                  (  B1_Matrix^T,   Gamma * I,     0      )   N
			*                  (  B2_Matrix^T,        0       Gamma *I )   M
			*
			* Quadric Vector   (D0_Vector)  3
			*                  (D1_Vector)  N
			*                  (D2_Vector)  M
			*
			* Quadric Scalar    CScalar     size 1
			*/
			// quadric matrix
			SymmetricMatrix  CMatrix;   // 3x3  - Symmetric
			B1MatrixType     B1Matrix;  // 3xN  - BasicAttr Gradients
			B2MatrixType     B2Matrix;  // 3xM  - (sparse) AddionalAttr Gradients
			double           Gamma = 0.; // diagonal identity in quadric matrix
            // quadric vector
			D0VectorType    D0Vector; // 3
			D1VectorType    D1Vector; // N
			D2VectorType    D2Vector; // M (sparse)

		    // quadric scalar
			double          CScalar = 0.;

			// volume constraint
			double          VolDistConstraint = 0.;
			Vec3d           VolGradConstraint;

			// used to clamp 1st UV
			FAABBox2d  UVBBox;
		};


		// Wedge type Quadric that contains two arrays of attributes.
		// The first is dense, and the second may be sparse.
		template <typename SimpVertexType_, typename SparseWeightArrayType_>
		class TFaceQuadric : public TQuadricBase<typename SimpVertexType_::BasicAttrContainerType, typename SimpVertexType_::AttrContainerType>
		{
		public:
			typedef TQuadricBase<typename SimpVertexType_::BasicAttrContainerType, typename SimpVertexType_::AttrContainerType>   MyBase;

			typedef SimpVertexType_                                            SimpVertexType;
			typedef typename SimpVertexType::DenseAttrAccessor                 DenseAttrAccessor;
			typedef typename SimpVertexType::AttrContainerType                 AttrContainerType;
			typedef typename SimpVertexType::BasicAttrContainerType            BasicAttrContainerType;

			typedef SparseWeightArrayType_                                     SparseWeightContainerType;

			typedef typename MyBase::B1MatrixType                              B1MatrixType;
			typedef typename MyBase::B2MatrixType                              B2MatrixType;
			typedef typename MyBase::D0VectorType                              D0VectorType;
			typedef typename MyBase::D1VectorType                              D1VectorType;
			typedef typename MyBase::D2VectorType                              D2VectorType;

			using MyBase::CMatrix;
			using MyBase::B1Matrix;
			using MyBase::B2Matrix;
			using MyBase::Gamma;
			using MyBase::D0Vector;
			using MyBase::D1Vector;
			using MyBase::D2Vector;
			using MyBase::CScalar;
			using MyBase::VolGradConstraint;
			using MyBase::VolDistConstraint;
			using MyBase::UVBBox;

			/**
			* @param TriVert0  - The three verts that define the triangle face.
			* @param TriVert1  -
			* @param TriVert2  -
			*                    NB: These have to be some form of TSkeletalSimpVerts
			*
			* @param BasicWeights       - The weights to be applied to the BasicAttributes
			* @param AdditionalWeights  - The weights to be applied to the AdditionalAttributes
			*/
			TFaceQuadric(const SimpVertexType& TriVert0,
				         const SimpVertexType& TriVert1,
				         const SimpVertexType& TriVert2,
				         const D1VectorType& BasicWeights,
				         const SparseWeightContainerType& AdditionalWeights,
						 const bool bUseLegacyAttrGrad);


			TFaceQuadric() = default;
			TFaceQuadric(const TFaceQuadric& Other) = default;
			TFaceQuadric& operator=(const TFaceQuadric& Other) = default;

			/**
			*  Returns the quadric evaluation at this point  (vTr . A . v  + 2 bTr. v + c)  where A = quadric matrix, b = DVector, c = CScalar
			*                                           with bra-ket  <v|A v>  + 2 <b | v> + c
			*/
			double Evaluate(const SimpVertexType& Vert, const D1VectorType& BasicWeights, const SparseWeightContainerType& AdditionalWeights) const;
		
			/**
			* @param Vert - Update Vert.BasicAttributes and Vert.AdditionalAttributes to the values interpolated at Vert.Position.
			*               NB: assumes Vert.Position is already defined
			*
			* @param BasicWeights       - The weights to be applied to the BasicAttributes
			* @param AdditionalWeights  - The weights to be applied to the AdditionalAttributes
			*/
			void CalcAttributes(SimpVertexType& Vert, const D1VectorType& BasicWeights, const SparseWeightArrayType_& AdditionalWeights) const;

			TFaceQuadric& operator+=(const TFaceQuadric& Other)
			{
				MyBase::operator+=(Other);
				return *this;
			}

			double TotalArea() const { return Gamma; }
		private:
			/**
			* -- Private: Used in the constructor --
			*
			* Computes the Gradient coefficient for each attribute index and stores them in the ResultMatrix.  The resulting "distances" are stored in the result vector
			*
			* @param GradientTool  - Tool used to solve for the interpolating gradient and distances    (position0^t , 1)  ( g_i[0] )  = (vert0Attr[i])
			*                                                                                           (position1^t , 1)  ( g_i[1] )    (vert1Attr[i])
			*                                                                                           (position2^t,  1)  ( g_i[2] )    (vert2Attr[i])
			*                                                                                           (faceNormal^t, 0)  (  d_i )      (     0      )
			* @param Vert0Attrs - Attribute containers for each vertex in the triangle
			* @param Vert1Attrs
			* @param Vert2Attrs
			*
			* @param Weights    - Weights to scale the attributes with.
			*
			* @param GradientMatrix  - Matrix with columns that are the "-gradient"  (-g_i) of attributes
			* @param DistanceVector  - The "-distances"  (-d_i) for each attribute.
			*
			* @return 'false' if the gradient encoding failed and the Gradient Matrix is empty.  In either event, the Distance Vector holds useful stuff.
			*/
			template <typename VectorType, typename MatrixType, typename WeightsArrayType>
			bool    EncodeAttrGradient(const InverseGradientProjection& GradientTool,
				                       const VectorType& Vert0Attrs,
				                       const VectorType& Vert1Attrs,
				                       const VectorType& Vert2Attrs,
				                       const WeightsArrayType& Weights,
				                       MatrixType& GradientMatrix,
				                       VectorType& DistanceVector);

			/**
			* -- Private: Used in the constructor --
			*
			* Weight the quadric values by an area.
			*/
			void WeightByArea(double Area);

			// Helpers
			/**
			*  If the weight > 1.e-6  then compute the interpolated attribute values from the gradient matrix and distances.
			*  otherwise set the attr to zero;
			*
			*  Attr[i] =  1 / (weight(i) * area)  * ( < Pos | GradientMatrix.GetColumn(i) > + Dist(i) )
			*/
			
			static void ComputeAttrs(const double Area, const B1MatrixType&  GradientMatrix, const D1VectorType&  DistVector, const Vec3d& Pos, const D1VectorType&  Weights, DenseAttrAccessor& Attrs);
			static void ComputeAttrs(const double Area, const SparseBMatrix& GradientMatrix, const SparseVecD& DistVector, const Vec3d& Pos, const SparseWeightContainerType& Weights, SparseVecD& Attrs);

			static void SumOuterProducts(const B1MatrixType& GradientArray, const D1VectorType& DistArray, SymmetricMatrix& OuterProductSum, Vec3d& DistGradientSum);
			static void SumOuterProducts(const SparseBMatrix& GradientArray, const SparseVecD& DistArray, SymmetricMatrix& OuterProductSum, Vec3d& DistGradientSum);

		};



		// Tool used to accumulate the quadric values and find the optimal position.
	
		template <typename FaceQuadricType>
		class TQuadricOptimizer : public TQuadricBase<typename FaceQuadricType::BasicAttrContainerType, typename FaceQuadricType::AttrContainerType>
		{
		public:
			typedef typename FaceQuadricType::SimpVertexType                           SimpVertexType;
			typedef typename FaceQuadricType::AttrContainerType                        AttrContainerType;
			typedef typename FaceQuadricType::BasicAttrContainerType                   BaseAttrContainerType;
			typedef TQuadricBase<BaseAttrContainerType, AttrContainerType>             MyBase;
			typedef FaceQuadricType                                                    FFaceQuadric;

			using MyBase::CMatrix;
			using MyBase::B1Matrix;
			using MyBase::B2Matrix;
			using MyBase::Gamma;
			using MyBase::D0Vector;
			using MyBase::D1Vector;
			using MyBase::D2Vector;
			using MyBase::CScalar;
			using MyBase::VolGradConstraint;
			using MyBase::VolDistConstraint;
			using MyBase::UVBBox;

			TQuadricOptimizer() :
				MyBase()
			{}

			void AddFaceQuadric(const FFaceQuadric& FaceQuadric)
			{
				MyBase::operator+=(FaceQuadric);
			};

			void AddEdgeQuadric(const FEdgeQuadric& EdgeQuadric)
			{
				// The edge quadric only hold the geometric parts of a quadric
				CMatrix  += EdgeQuadric.CMatrix;
				D0Vector += EdgeQuadric.D0Vector;
				CScalar  += EdgeQuadric.CScalar;
			}

			/**
			@param OptimalPosition  - Result of optimizing the quadric

			@param bPreserveVolume  - Controls the use of the volume conservation fixup.
			@param VolumeImportance - Artificial scalar between 0, 1 that scales the amount of conservation fixup actually used.
			                          the value 0 corresponds to no fixup, and 1 to full fixup
			*/
			bool Optimize(Vec3d& OptimalPosition, const bool bPreserveVolume, const double VolumeImportance) const;


		};


		// ---  Implementations. 


		// ---- Helpers

		static void InitializeBMatrix(const TArray<int32>& IterMask, SparseBMatrix& Matrix)
		{
			Matrix.Reset();
			for (int i = 0, iMax = IterMask.Num(); i < iMax; ++i)
			{
				if (IterMask[i] == 0) continue;

				Matrix.SetColumn(i, Vec3d(0., 0., 0.));
			}
		}

		template <int32 SIZE>
		static void InitializeBMatrix(const DenseIterMask<SIZE>& IterMask, TDenseBMatrix<SIZE>& Matrix)
		{
			// Zero Matrix
			Matrix.Reset();
		}


		static void InitializeBVector(const TArray<int32>& IterMask, SparseVecD& Vec)
		{
			Vec.Reset();
			for (int i = 0, iMax = IterMask.Num(); i < iMax; ++i)
			{
				if (IterMask[i] == 0) continue;

				Vec.SetElement(i, 0.);

			}
		}

		template <int32 SIZE>
		static void InitializeBVector(const DenseIterMask<SIZE>& IterMask, TDenseVecD<SIZE>& Vec)
		{
			// Zero Vector
			Vec.Reset();
		}



		// --- Quadric Base

		template <typename BasicType, typename AddType>
		inline double TQuadricBase<BasicType, AddType>::EvaluateQaudric(const D0VectorType& Pos, const D1VectorType& S1, const D2VectorType& S2) const
		{
			// S = {pos, s1, s2}

			// < s | Quadric_Matrix * s> = 
			//                            < pos | C * pos> + 2 ( < pos | B1 * s1 > + < pos | B2 * s2 >  ) + Gamma* ( < s1 | s1 > + < s2 | s2 > )
			double  SQmS = 0.;
			{
				double Pt1 = Pos.DotProduct(CMatrix * Pos);
				double Pt2 = 2. * Pos.DotProduct(B1Matrix * S1 + B2Matrix * S2);
				double Pt3 = Gamma * ( S1.L2NormSqr() + S2.L2NormSqr() );

				SQmS = Pt1 + Pt2 + Pt3;
			}

			// quadric_vector = {D0Vector, D1Vector, D2Vector}
			//
			// 2. * < s | quadratic_vector > = 
			//                                2.* (  < pos | D0Vector > + < s1 | D1Vector > + < s2 | D2Vector > )
			double CrossTerm = 0.;
			{
				double Pt1 = Pos.DotProduct(D0Vector);
				double Pt2 =  S1.DotProduct(D1Vector);
				double Pt3 =  S2.DotProduct(D2Vector);

				CrossTerm = 2. * (Pt1 + Pt2 + Pt3);
			}

			// < s | Quadric_Matrix * s > + 2 < s | quadric_vector > + quadric_scalar

			return SQmS + CrossTerm + CScalar;
		}

		template <typename BasicType, typename AddType>
		inline TQuadricBase<BasicType, AddType>& TQuadricBase<BasicType, AddType>::operator+=(const TQuadricBase<BasicType, AddType>& Other)
		{
			CMatrix += Other.CMatrix;

			B1Matrix += Other.B1Matrix;
			B2Matrix += Other.B2Matrix;

			D0Vector += Other.D0Vector;
			D1Vector += Other.D1Vector;
			D2Vector += Other.D2Vector;

			CScalar += Other.CScalar;

			Gamma += Other.Gamma;

			VolGradConstraint += Other.VolGradConstraint;
			VolDistConstraint += Other.VolDistConstraint;

			UVBBox += Other.UVBBox;

			return *this;
		}

		template <typename BasicType, typename AddType>
		inline TQuadricBase<BasicType, AddType>& TQuadricBase<BasicType, AddType>::operator=(const TQuadricBase<BasicType, AddType>& Other)
		{
			CMatrix = Other.CMatrix;

			B1Matrix = Other.B1Matrix;
			B2Matrix = Other.B2Matrix;

			D0Vector = Other.D0Vector;
			D1Vector = Other.D1Vector;
			D2Vector = Other.D2Vector;

			CScalar = Other.CScalar;

			Gamma = Other.Gamma;

			VolGradConstraint = Other.VolGradConstraint;
			VolDistConstraint = Other.VolDistConstraint;

			UVBBox = Other.UVBBox;

			return *this;
		}

		// ---- FEdgeQuadirc --- 


		inline FEdgeQuadric::FEdgeQuadric(const Vec3d& Vert0Pos, const Vec3d& Vert1Pos, const Vec3d& FaceNormal, const double EdgeWeight)
		{
		
			// Early out if the face normal doesn't have unit length.
			// In practice the normal was computed in single precision, and may have failed if the triangle was degenerate.
			// This test exists largely to catch the case where the normal failed.
			// NB: this threshold is only 0.01  
			{
				double LenghtSqrd = FaceNormal.LengthSqrd();
				bool bIsUnitLenght = (FMath::Abs(LenghtSqrd - 1.) < THRESH_VECTOR_NORMALIZED);
				if (!bIsUnitLenght)
				{
					return;
				}

			}


			Vec3d Edge = Vert1Pos - Vert0Pos;

			// Weight scaled on edge length 

			const double EdgeLength  = std::sqrt(Edge.LengthSqrd());
			const double Weight = EdgeWeight * EdgeLength;

			if (EdgeLength < 1.e-8)
			{
				return;
			}
			else
			{
				Edge *= 1. / EdgeLength;
			}

			// Normal that is perpendicular to the edge, and face.  The constraint should try
			// to keep points on the plane associated with this constraint plane.
			Vec3d N = CrossProduct(Edge, FaceNormal);

			// Make the n unit length and record the old length for use in computing the area.
			double Length = N.LengthSqrd();
			if (Length < 1.e-8)
			{
				// @todo make sure everything is Zero();
				return;
			}
			else
			{
				// normalize
				N *= 1. / std::sqrt(Length);
			}

			double Dist = -N.DotProduct(Vert0Pos);  // - n.dot.p0

			CMatrix  = ScaledProjectionOperator(N); // N . NTranspose
			D0Vector = Dist * N;                    // Dist * N;
			CScalar  = Dist * Dist;                 // Dist * Dist

												   // Scale by weight

			CMatrix  *= Weight;
			D0Vector *= Weight;
			CScalar  *= Weight;
		}

		inline double FEdgeQuadric::Evaluate(const Vec3d& Pos) const
		{
			// < pos | C_Matrix pos > + 2 < pos | D0_Vector > + CScalar
			Vec3d CmPos = CMatrix * Pos;

			double Result = Pos.DotProduct(CmPos) + 2. * Pos.DotProduct(D0Vector) + CScalar;
			return Result;
		}

		inline FEdgeQuadric& FEdgeQuadric::operator+=(const FEdgeQuadric& Other)
		{
			CMatrix  += Other.CMatrix;
			D0Vector += Other.D0Vector;
			CScalar  += Other.CScalar;
			return *this;
		}


		// --- Face Quadric

		template <typename SimpVertexType, typename SparseWeightArrayType>
		TFaceQuadric<SimpVertexType, SparseWeightArrayType>::TFaceQuadric(const SimpVertexType& TriVert0, const SimpVertexType& TriVert1, const SimpVertexType& TriVert2,
			                                                              const D1VectorType& BasicWeights, const SparseWeightContainerType& AdditionalWeights,
																		  const bool bUseLegacyAttrGrad)
		{

			// Convert point locations to double.

			const Vec3d Vert0Pos(TriVert0.GetPos());
			const Vec3d Vert1Pos(TriVert1.GetPos());
			const Vec3d Vert2Pos(TriVert2.GetPos());

			// The normal direction ( but not necessarily unit length)
			Vec3d FaceNormal = CrossProduct(Vert2Pos - Vert0Pos, Vert1Pos - Vert0Pos); //  n = (p2 - p0) ^ (p1 - p0);

			
			// Make the FaceNormal unit length and record the old length for use in computing the area.
			double LengthSqrd = FaceNormal.LengthSqrd();
			double Area;  // = 1/2 | edge X edge |

			if (LengthSqrd < SMALL_NUMBER)
			{   
				// Face was too small.
				// Allocate defaults for the quadric and return.

				D1Vector.Reset();
				B1Matrix.Reset();

				// @todo make sure everything is Zero();
				return;
			}
			else
			{
				// normalize
				double Length = std::sqrt(LengthSqrd);
				FaceNormal *= 1. / Length;
				Area = 0.5 * Length;   // = 1/2 | edge X edge |
			}

			// Dist + FaceNormal.Dot.P = 0;
			// for any p on the face.
			double Dist = -FaceNormal.DotProduct(Vert0Pos);  // - n.dot.p0


	       // For volume constraint:  FaceNormal.Dot(Pos) + Dist = 0 

			VolGradConstraint = FaceNormal * (1. / 3.);   // this 1/3 is shared by all faces, it ultimately drops out... 
			VolDistConstraint = Dist * (1. / 3.);

			// Initialize parts of the quadric with the Geometric component

			CMatrix  = ScaledProjectionOperator(FaceNormal);  // N . NTranspose
			D0Vector = Dist * FaceNormal;                     // Dist * N;
			CScalar  = Dist * Dist;                           // Dist * Dist


															// create the tool needed to compute the gradient coefficients of the attributes.

			InverseGradientProjection  GradientTool(Vert0Pos,
													Vert1Pos,
													Vert2Pos,
													FaceNormal,
													bUseLegacyAttrGrad);

			// Accumulate the terms related to the gradient of the Basic Attributes
			{
				// Update the UV BBox
				UVBBox.ExpandToInclude(TriVert0.BasicAttributes.TexCoords[0]);
				UVBBox.ExpandToInclude(TriVert1.BasicAttributes.TexCoords[0]);
				UVBBox.ExpandToInclude(TriVert2.BasicAttributes.TexCoords[0]);



				// Construct double precision arrays from the float data held by the vertex.
				const D1VectorType Vert0BasicAttrs(TriVert0.GetBasicAttrAccessor());
				const D1VectorType Vert1BasicAttrs(TriVert1.GetBasicAttrAccessor());
				const D1VectorType Vert2BasicAttrs(TriVert2.GetBasicAttrAccessor());

				const bool bHasGradients = EncodeAttrGradient(GradientTool, Vert0BasicAttrs, Vert1BasicAttrs, Vert2BasicAttrs, BasicWeights, B1Matrix, D1Vector);

				CScalar += D1Vector.L2NormSqr();


				if (bHasGradients)
				{
					// Accumulate the outer product of the attribute gradient in the CMatrix:   CMatrix += Sum ( outer_product(B1Matrix.GetColumn(i)) )  
					// Accumulate the scaled attribute distance vector in D0Vector:             D0Vector += Sum( D1[i] * B1Matrix.GetColumn(i) )                                        
					SumOuterProducts(B1Matrix, D1Vector, CMatrix, D0Vector);
				}

			}

			// Accumulate the terms related the the Additional Attributes.
			{

				const D2VectorType& Vert0AdditionalAttrs = TriVert0.GetAdditionalAttrContainer();
				const D2VectorType& Vert1AdditionalAttrs = TriVert1.GetAdditionalAttrContainer();
				const D2VectorType& Vert2AdditionalAttrs = TriVert2.GetAdditionalAttrContainer();


				const bool bHasGradeints = EncodeAttrGradient(GradientTool, Vert0AdditionalAttrs, Vert1AdditionalAttrs, Vert2AdditionalAttrs, AdditionalWeights, B2Matrix, D2Vector);

				CScalar += D2Vector.L2NormSqr();

				if (bHasGradeints)
				{
					// Accumulate the outer product of the attribute gradient in the CMatrix:   CMatrix += Sum ( outer_product(B1Matrix.GetColumn(i)) )  
					// Accumulate the scaled attribute distance vector in D0Vector:             D0Vector += Sum( D1[i] * B1Matrix.GetColumn(i) )                                        
					SumOuterProducts(B2Matrix, D2Vector, CMatrix, D0Vector);
				}

			}
			// Store the area
			Gamma = Area;

			// Weight the quadric matrix, vector, and scalar by area
			WeightByArea(Area);

		}

		template <typename SimpVertexType, typename SparseWeightArrayType>
		template <typename VectorType, typename MatrixType, typename WeightsArrayType>
		bool   TFaceQuadric<SimpVertexType, SparseWeightArrayType>::EncodeAttrGradient( const InverseGradientProjection& GradientTool,
			                                                                            const VectorType& Vert0Attrs,
			                                                                            const VectorType& Vert1Attrs,
			                                                                            const VectorType& Vert2Attrs,
			                                                                            const WeightsArrayType& Weights,
			                                                                            MatrixType& GradientMatrix,
			                                                                            VectorType& DistanceVector)
		{
			// Need the union of the possibility sparse attrs iteration range.

			auto IterMask = SkeletalSimplifier::LinearAlgebra::GetIterationMask(Vert0Attrs, Vert1Attrs, Vert2Attrs);

			// Initialize the BMatrix (This holds -AttrGrad) and B2Vector (holds -AttrDist) 

			InitializeBMatrix(IterMask, GradientMatrix);
			InitializeBVector(IterMask, DistanceVector);


			const bool bCanFindGradients = GradientTool.IsValid();
			// compute the gradient coefficients for each attribute,
			// storing the results in the BMatrix and the associated distances in the B2Vector

			if (bCanFindGradients)  // non-degenerate triangle.
			{

				Vec3d PerVertexData;
				for ( int32 i = 0; i < IterMask.Num(); ++i )
				{
					// no non-zero element 
					if (IterMask[i] == 0) continue;

					const double weight = Weights.GetElement(i);
					if (weight < 1.e-6) continue;

					// get the data for this attribute at the three corners of the triangle

					PerVertexData[0] = Vert0Attrs.GetElement(i);
					PerVertexData[1] = Vert1Attrs.GetElement(i);
					PerVertexData[2] = Vert2Attrs.GetElement(i);
					PerVertexData *= weight;

					// Compute the Attribute Grad and AttrDist
					Vec3d AttrGrad; 
					double AttrDist = GradientTool.ComputeGradient(PerVertexData, AttrGrad);

					// Store in the GradientMatrix.
					GradientMatrix.SetColumn(i, -1. * AttrGrad);

					// Store in Distance Vector
					DistanceVector.SetElement(i, -1. * AttrDist);

				}
			}
			else
			{
				// If the triangle was degenerate (the gradient tool will be invalid), then just use the average
				// of the attributes and implicitly AttrGrad = 0;
				Vec3d PerVertexData;
				for ( int32 i = 0; i < IterMask.Num(); ++i )
				{
					// no non-zero element 
					if (IterMask[i] == 0) continue;

					const double weight = Weights.GetElement(i);
					if (weight < 1.e-6) continue;


					// get the data for this attribute at the three corners of the triangle

					PerVertexData[0] = Vert0Attrs.GetElement(i);
					PerVertexData[1] = Vert1Attrs.GetElement(i);
					PerVertexData[2] = Vert2Attrs.GetElement(i);
					PerVertexData *= weight;

					double AverageAttr = (1. / 3.) * (PerVertexData[0] + PerVertexData[1] + PerVertexData[2]);

					GradientMatrix.SetColumn(i, Vec3d(0., 0., 0.));
					DistanceVector.SetElement(i, -1. * AverageAttr);

				}
			}

			return bCanFindGradients;
		}

		template <typename SimpVertexType, typename SparseWeightArrayType>
		void TFaceQuadric<SimpVertexType, SparseWeightArrayType>::WeightByArea(double Area)
		{
			// Weight by area

			CMatrix  *= Area;
			B1Matrix *= Area;
			B2Matrix *= Area;

			D0Vector *= Area;
			D1Vector *= Area;
			D2Vector *= Area;
			CScalar  *= Area;

			VolDistConstraint *= Area;
			VolGradConstraint *= Area;

		}

		template <typename SimpVertexType, typename SparseWeightArrayType>
		inline void TFaceQuadric<SimpVertexType, SparseWeightArrayType>::ComputeAttrs(const double Area, const B1MatrixType&  GradientMatrix, const D1VectorType&  DistVector, const Vec3d& Pos, const D1VectorType&  Weights, DenseAttrAccessor& Attrs)
		{

			checkSlow(Attrs.Num() == DistVector.Num());

			for (int32 i = 0; i < DistVector.Num(); ++i)
			{
				double weight = Weights.GetElement(i);
				double AttrValue = 0.;
				if (!(weight < 1.e-6))
				{                              // GradientMatrix.GetColum(i) = -g_i  // DistVector.GetElement(i) = -d_i 
					AttrValue = Pos.DotProduct( GradientMatrix.GetColumn(i) ) + DistVector.GetElement(i);
					AttrValue /= (weight * Area);
				}

				Attrs[i] = -AttrValue;
			}
		}

		template <typename SimpVertexType, typename SparseWeightArrayType>
		inline void TFaceQuadric<SimpVertexType, SparseWeightArrayType>::ComputeAttrs(const double Area, const SparseBMatrix& GradientMatrix, const SparseVecD& DistVector, const Vec3d& Pos, const SparseWeightContainerType& Weights, SparseVecD& Attrs)
		{
			Attrs.Reset(); // Empty this sparse data structure.
			for (auto citer = GradientMatrix.GetData().CreateConstIterator(); citer; ++citer)
			{
				int32 idx = citer->Key;

				double weight = Weights.GetElement(idx);
				double AttrValue = 0.;
				if (!(weight < 1.e-6))
				{
					AttrValue = Pos.DotProduct(citer->Value) + DistVector.GetElement(idx);
					AttrValue /= (weight * Area);
				}

				Attrs.SetElement(idx, -AttrValue);
			}
		}

		template <typename SimpVertexType, typename SparseWeightArrayType>
		inline void TFaceQuadric<SimpVertexType, SparseWeightArrayType>::CalcAttributes(SimpVertexType& Vert, const D1VectorType& BasicWeights, const SparseWeightArrayType& AdditionalWeights) const
		{
			// convert to double precision.

			const Vec3d Pos = Vert.GetPos();

			// Update the basic attrs

			auto BasicAttrAccessor = Vert.GetBasicAttrAccessor();
			ComputeAttrs(Gamma, B1Matrix, D1Vector, Pos, BasicWeights, BasicAttrAccessor);

			// Clamp first UV channel to a slightly padded version of the UV support. 
			const float PaddingFactor = 0.2f;
			UVBBox.ClampPoint(Vert.BasicAttributes.TexCoords[0], PaddingFactor);


			// Update the Additional Attrs

			auto& AddionalAttrs = Vert.GetAdditionalAttrContainer();
			ComputeAttrs(Gamma, B2Matrix, D2Vector, Pos, AdditionalWeights, AddionalAttrs);
		}

		template <typename SimpVertexType, typename SparseWeightArrayType>
		inline double TFaceQuadric<SimpVertexType, SparseWeightArrayType>::Evaluate(const SimpVertexType& Vert, const D1VectorType& BasicWeights, const SparseWeightArrayType& AdditionalWeights) const
		{

			// Convert position to double

			const Vec3d Pos = Vert.GetPos();

			// Need scaled version of the state

			const auto BasicAttrsAccessor = Vert.GetBasicAttrAccessor();
			D1VectorType S1;

			checkSlow(BasicAttrsAccessor.Num() == BasicWeights.Num());
			checkSlow(S1.Num() == BasicAttrsAccessor.Num());

			for (int32 i = 0; i < S1.Num(); ++i)
			{
				S1.SetElement(i, BasicAttrsAccessor[i] * BasicWeights[i]);
			}

			const auto& AdditionalAttrs = Vert.GetAdditionalAttrContainer();
			D2VectorType  S2;


			for (auto citer = AdditionalAttrs.GetData().CreateConstIterator(); citer; ++citer)
			{
				const int32 idx = citer->Key; 
				double Value = citer->Value;

				double weight = AdditionalWeights.GetElement(idx);
				S2.SetElement(idx, weight * Value);
			}

			// Evaluate the quadric using the scaled state values.
			return MyBase::EvaluateQaudric(Pos, S1, S2);
		}

		template <typename SimpVertexType, typename SparseWeightArrayType>
		inline void TFaceQuadric<SimpVertexType, SparseWeightArrayType>::SumOuterProducts(const B1MatrixType& GradientArray, const D1VectorType& DistArray, SymmetricMatrix& OuterProductSum, Vec3d& DistGradientSum)
		{

			checkSlow(DistArray.Num() == GradientArray.NumCols());

			const int32 NumElements = DistArray.Num();
			for (int32 i = 0; i < NumElements; ++i)
			{
				double Dist = DistArray[i];
				const Vec3d& Gradient = GradientArray.GetColumn(i);

				DistGradientSum += Dist * Gradient;

				OuterProductSum += ScaledProjectionOperator(Gradient);

			}
		}

		template <typename SimpVertexType, typename SparseWeightArrayType>
		inline void TFaceQuadric<SimpVertexType, SparseWeightArrayType>::SumOuterProducts(const SparseBMatrix& GradientArray, const SparseVecD& DistArray, SymmetricMatrix& OuterProductSum, Vec3d& DistGradientSum)
		{

			const auto&  SparseData = DistArray.GetData();
			for (auto citer = SparseData.CreateConstIterator(); citer; ++citer)
			{
				double Dist = citer->Value;
				const Vec3d& Gradient = GradientArray.GetColumn(citer->Key);

				DistGradientSum += Dist * Gradient;

				OuterProductSum += ScaledProjectionOperator(Gradient);

			}
		}

		//--- Optimizer

		template <typename FaceQuadricType>
		inline bool TQuadricOptimizer<FaceQuadricType>::Optimize(Vec3d& OutPosition, const bool bPreserveVolume, const double VolumeImportance) const
		{
			/**
			* Optimizing the quadric requires inverting a matrix system:
			*
			*                         3            N          M
			* Quadric Matrix    ( C_Matrix,    B1_Matrix, B2_Matrix )   3
			*                   ( B1_Matrix^T, Gamma * I,    0      )   N
			*                   ( B2_Matrix^T,     0    ,  Gamma *I )   M
			*
			* Quadric Vector            ( D0_Vector )  3
			*                           ( D1_Vector )  N
			*                           ( D2_Vector )  M
			*
			*  Solve:
			*     Quadric_Matrix * Vector = -Quadric_Vector
			*
			* for Vector.    Here Vector = (  Pos )  3
			*                              (   S1 )  N
			*                              (   S2 )  M
			* with "Pos" being the position and S1 & S2 are state for the dense (S1) and sparse (S2) attributes.
			*
			* To solve, a little manipulation shows that the state can be computed from the position
			*
			*    S1 = -1/gamma ( D1_Vector + B1_Matrix^T * Pos )
			*    S2 = -1/gamma ( D2_Vector + B2_Matrix^T * Pos )
			*
			*  and the position can be obtained by inverting a symmetric 3x3 matrix,
			*
			*  [ C_Matrix - 1/gamma (B1_Matrix * B1_Matrix^T + B2_Matrix * B2_Matrix^T ) ] * Pos =
			*                                     1/gamma [B1_Matrix * D1_Vector + B2_Matrix * D2_Vector] - D0_Vector
			*
			*   Lhs_Matrix * Pos = Rhs_Vector
			*
			*      The optimal position is given by   Pos = Inverse(Lhs_Matrix) * Rhs_Vector
			*
			*  If Volume Preservation is desired, a scalar Lagrange multiplier 'lm' is used to inflate the system
			*
			*
			*            3                 1
			*      ( Lhs_Matrix,      Vol_Gradient )  ( Pos )    = ( Rhs_Vector     )
			*      ( Vol_Gradient^T,       0       )  (  lm )      ( -Vol_Grad_Dist )
			*
			*
			*     lm  =  (Vol_Grad_Dist +  <Vol_Grad | InvL * RHS_Vector>  ) / <Vol_Grad | InvL * Vol_Grad>
			*
			*     Pos  = InvL * Rhs_Vec - lm InvL * Vol_Grad
			*
			*     where InvL = Lhs_Matrix.Inverse()
			*
			*     Note:  InvL * Rhs_Vec is the unconstrained solution (if you ignored volume preservation)
			*            and  -lm * InvL * Vol_Grad   is the correction.
			*
			*    notation:
			*    <VectorA | VectorB> = DotProduct(VectorA, VectorB).
			*/


			const double Threshold = 1.e-12;
			
			if (Gamma < Threshold) return false;

			// LHS

			SymmetricMatrix LhsMatrix = CMatrix - (1. / Gamma) * ( OuterProductOperator(B1Matrix) + OuterProductOperator(B2Matrix) );

			// Invert:

			bool bSucces = true;
			const SymmetricMatrix InvLhsMatrix = LhsMatrix.Inverse(bSucces, Threshold);

			// Return false if we can't find any optimal position.

			if (bSucces)
			{

				// RHS

				Vec3d RhsVector = (1. / Gamma) * (B1Matrix * D1Vector + B2Matrix * D2Vector) - D0Vector;

				// The optimal position with out the volume constraint

				OutPosition = InvLhsMatrix * RhsVector;

				if (bPreserveVolume)
				{
					const Vec3d InvLhsGVol      = InvLhsMatrix * VolGradConstraint;
					const double GVolInvLhsGVol = VolGradConstraint.DotProduct(InvLhsGVol);

					// Check that the constraint can be satisfied.
					if (FMath::Abs(GVolInvLhsGVol) > Threshold)
					{
						const double LagrangeMultiplier = (1. / GVolInvLhsGVol) * (VolDistConstraint + VolGradConstraint.DotProduct(OutPosition));
						const Vec3d VolumeCorrection    = -LagrangeMultiplier * InvLhsGVol;

						OutPosition += VolumeImportance * VolumeCorrection;
					}
				}
			}
			return bSucces;
		}

	}
}
