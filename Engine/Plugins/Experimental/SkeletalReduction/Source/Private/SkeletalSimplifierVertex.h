// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SkeletalSimplifierLinearAlgebra.h"

#include "MeshBuild.h"      // ApprxEquals for normals & UVs

namespace SkeletalSimplifier
{
	namespace VertexTypes
	{
		using namespace SkeletalSimplifier::LinearAlgebra;

		/**
		* Class that holds the dense vertex attributes.
		* Normal, Tangent, BiTangent, Color, TextureCoords
		*
		* NB: this can be extended with additional float-type attributes.
		*/
		template <int32 NumTexCoords>
		class TBasicVertexAttrs
		{
		public:
			enum { NumUVs = NumTexCoords};

			// NB: if you update the number of float-equivalent attributes, the '13' will have to be updated.
			typedef TDenseVecD<13 + 2 * NumTexCoords>      DenseVecDType;
			
			typedef DenseVecDType                          WeightArrayType;
			typedef TDenseBMatrix<13 + 2 * NumTexCoords>   DenseBMatrixType;

			// NB: Required that these all have float storage.
			// - Base Attributes: size = 13 + 2 * NumTexCoord

			FVector3f		Normal;      // 0, 1, 2
			FVector3f		Tangent;     // 3, 4, 5
			FVector3f       BiTangent;   // 6, 7, 8
			FLinearColor	Color;       // 9, 10, 11, 12
			FVector2f		TexCoords[NumTexCoords];  // 13, .. 13 + NumTexCoords * 2 - 1


			// used to manage identity of split/non-split vertex attributes.
			// note, could have external storage for the above attributes and just use these to point to those values.
			struct FElementIDs
			{
				int32 NormalID;
				int32 TangentID;
				int32 BiTangentID;
				int32 ColorID;
				int32 TexCoordsID[NumTexCoords];

				enum { InvalidID = -1 };

				// construct with invalid ID values.
				FElementIDs()
				{
					NormalID = InvalidID;
					TangentID = InvalidID;
					BiTangentID = InvalidID;
					ColorID = InvalidID;
					for (int32 i = 0; i < NumTexCoords; ++i)
					{
						TexCoordsID[i] = InvalidID;
					}
				}

				FElementIDs(const  FElementIDs& other)
				{
					NormalID = other.NormalID;
					TangentID = other.TangentID;
					BiTangentID = other.BiTangentID;
					ColorID = other.ColorID;
					for (int32 i = 0; i < NumTexCoords; ++i)
					{
						TexCoordsID[i] = other.TexCoordsID[i];
					}
				}

				// Subtract other ID struct from this one.
				FElementIDs operator-(const  FElementIDs& other) const
				{
					FElementIDs Result;
					Result.NormalID = NormalID - other.NormalID;
					Result.TangentID = TangentID - other.TangentID;
					Result.BiTangentID = BiTangentID - other.BiTangentID;
					Result.ColorID = ColorID - other.ColorID;
					for (int i = 0; i < NumTexCoords; ++i)
					{
						Result.TexCoordsID[i] = TexCoordsID[i] - other.TexCoordsID[i];
					}
					return Result;
				}

				// copy other ID if the mask value is zero
				void MaskedCopy(const  FElementIDs& IDMask, const  FElementIDs& other)
				{
					NormalID = (IDMask.NormalID == 0) ? other.NormalID : NormalID;
					TangentID = (IDMask.TangentID == 0) ? other.TangentID : TangentID;
					BiTangentID = (IDMask.BiTangentID == 0) ? other.BiTangentID : BiTangentID;
					ColorID = (IDMask.ColorID == 0) ? other.ColorID : ColorID;
					for (int i = 0; i < NumTexCoords; ++i)
					{
						TexCoordsID[i] = (IDMask.TexCoordsID[i] == 0) ? other.TexCoordsID[i] : TexCoordsID[i];
					}
				}

				// copy IDs values over to this for elemenets where IDMask == 0 and InverseIDMask != 0
				void MaskedCopy(const  FElementIDs& IDMask, const  FElementIDs& InverseIDMask, const  FElementIDs& BIDs)
				{
					if (InverseIDMask.NormalID != 0 && IDMask.NormalID == 0)
					{
						NormalID = BIDs.NormalID;
					}
					if (InverseIDMask.TangentID != 0 && IDMask.TangentID == 0)
					{
						TangentID = BIDs.TangentID;
					}
					if (InverseIDMask.BiTangentID != 0 && IDMask.BiTangentID == 0)
					{
						BiTangentID = BIDs.BiTangentID;
					}
					if (InverseIDMask.ColorID != 0 && IDMask.ColorID == 0)
					{
						ColorID = BIDs.ColorID;
					}
					for (int i = 0; i < NumTexCoords; ++i)
					{
						if (InverseIDMask.TexCoordsID[i] != 0 && IDMask.TexCoordsID[i] == 0)
						{
							TexCoordsID[i] = BIDs.TexCoordsID[i];
						}
					}
				}

			};
			
			FElementIDs ElementIDs;

		public:
			// vector semantic wrapper for raw array
			typedef TDenseArrayWrapper<float>                            DenseAttrAccessor;


			/**
			 * default construct to zero values
			 */
			TBasicVertexAttrs() :
				Normal(ForceInitToZero),
				Tangent(ForceInitToZero),
				BiTangent(ForceInitToZero),
				Color(ForceInitToZero)
			{
				for (int32 i = 0; i < NumTexCoords; ++i)
				{
					TexCoords[i] = FVector2f(ForceInitToZero);
				}
			}

			/**
			* copy construct
			*/
			TBasicVertexAttrs(const TBasicVertexAttrs& Other) :
				Normal(Other.Normal),
				Tangent(Other.Tangent),
				BiTangent(Other.BiTangent),
				Color(Other.Color),
				ElementIDs(Other.ElementIDs)
			{
				for (int32 i = 0; i < NumTexCoords; ++i)
				{
					TexCoords[i] = Other.TexCoords[i];
				}
			}

			/**
			* Number of float equivalents. 
			*/
			static int32 Size() { return (sizeof(TBasicVertexAttrs) - sizeof(FElementIDs)) / sizeof(float); /*( return 13 + 2 * NumTexCoords;*/ }

			/**
			* Get access to the data as a generic linear array of floats.
			*/
			DenseAttrAccessor  AsDenseAttrAccessor() { return DenseAttrAccessor((float*)&Normal, Size()); }
			const DenseAttrAccessor  AsDenseAttrAccessor() const { return DenseAttrAccessor((float*)&Normal, Size()); }

			/**
			* Assignment operator. 
			*/
			TBasicVertexAttrs& operator=(const TBasicVertexAttrs& Other)
			{
				DenseAttrAccessor MyData = AsDenseAttrAccessor();
				const DenseAttrAccessor OtherData = Other.AsDenseAttrAccessor();

				const int32 NumElements = MyData.Num();
				for (int32 i = 0; i < NumElements; ++i)
				{
					MyData[i] = OtherData[i];
				}

				return *this;
			}
			

			/**
			* Method to insure that the attribute values are valid by correcting any invalid ones.
			*/
			void Correct()
			{
				Normal.Normalize();
				Tangent -= FVector3f::DotProduct(Tangent, Normal) * Normal;
				Tangent.Normalize();
				BiTangent -= FVector3f::DotProduct(BiTangent, Normal) * Normal;
				BiTangent -= FVector3f::DotProduct(BiTangent, Tangent) * Tangent;
				BiTangent.Normalize();
				Color = Color.GetClamped();
			}

			/**
			* Strict equality test.
			*/
			bool  operator==(const TBasicVertexAttrs& Other)
			{
				bool bIsEqual =
					Tangent == Other.Tangent
					&& BiTangent == Other.BiTangent
					&& Normal == Other.Normal
					&& Color == Other.Color;

				for (int32 UVIndex = 0; bIsEqual && UVIndex < NumTexCoords; UVIndex++)
				{
					bIsEqual = bIsEqual && UVsEqual(TexCoords[UVIndex], Other.TexCoords[UVIndex]);
				}

				return bIsEqual;
			}

			/**
			* Approx equality test.
			*/
			bool IsApproxEquals(const TBasicVertexAttrs& Other) const
			{
				bool bIsApprxEqual =
					NormalsEqual(Tangent, Other.Tangent) &&
					NormalsEqual(BiTangent, Other.BiTangent) &&
					NormalsEqual(Normal, Other.Normal) &&
					Color.Equals(Other.Color);

				for (int32 UVIndex = 0; bIsApprxEqual && UVIndex < NumTexCoords; UVIndex++)
				{
					bIsApprxEqual = bIsApprxEqual && UVsEqual(TexCoords[UVIndex], Other.TexCoords[UVIndex]);
				}

				return bIsApprxEqual;
			}

		};

		/** Dynamic vertex attributes. Distinguished from the BoneSparseVertexAttrs. Based on SparseVecD, rather than
		 *  using dense vector wrappers, because of requirements by the simplifier. 
		 */
		class DynamicVertexAttrs : public SparseVecD
		{
		public:
			static void Correct()
			{
				// Do nothing.
			}
		};  

		/**
		* Sparse attributes.
		* Used to hold bone weights where the bone ID is the attribute key.
		*/
		class BoneSparseVertexAttrs : public SparseVecD
		{
		public:

			#define  SmallBoneSize 1.e-12  

			/**
			* Deletes smallest bones if currently more than MaxBoneNumber, and maintain the normalization of weight (all sum to 1).
			* keeping the bones sorted internally by weight (largest to smallest).
			*/
			void Correct( int32 MaxBoneNumber = 8)  
			{
				if (!bIsEmpty())
				{
				
					DeleteSmallBones();

					

					if (SparseData.Num() > MaxBoneNumber)
					{
						// sort by value from large to small
						SparseData.ValueSort([](double A, double B)->bool { return B < A; });

						SparseContainer Tmp;
						int32 Count = 0;
						for (const auto& BoneData : SparseData)
						{
							if (Count == MaxBoneNumber) break;
							Tmp.Add(BoneData.Key, BoneData.Value);
							Count++;
						}
						Swap(SparseData, Tmp);

					}

					SparseData.ValueSort([](double A, double B)->bool { return B < A; });

					Normalize();
				}
			}

			/**
			* Note: the norm here is the sum of weights (not L2 or L1 norm)
			*/
			void Normalize()
			{
				double SumOfWeights = SumValues();
				if (FMath::Abs(SumOfWeights) > 8 * SmallBoneSize)
				{
					double Inv = 1. / SumOfWeights;

					operator*=(Inv);
				}
				else
				{
					Empty();
				}
			}

			/**
			* Removes bones with very small weights. This may not be appropriate for all models.
			*/
			void DeleteSmallBones()
			{
				SparseContainer Tmp;
				for (const auto& BoneData : SparseData)
				{
					if (BoneData.Value > SmallBoneSize ) Tmp.Add(BoneData.Key, BoneData.Value);
				}
				Swap(SparseData, Tmp);
			}

			/**
			* Remove all bones.
			*/
			void Empty()
			{
				SparseContainer Tmp;
				Swap(SparseData, Tmp);
			}

			/**
			* Compare two sparse arrays.  A small bone weight will be equivalent to no bone weight.
			*/
			bool IsApproxEquals(const BoneSparseVertexAttrs& Other, double Tolerance = (double)KINDA_SMALL_NUMBER) const
			{

				SparseVecD::SparseContainer Tmp = SparseData;
				auto AddToElement = [&Tmp](const int32 j, const double Value)
				{
					double* Data = Tmp.Find(j);
					if (Data != nullptr)
					{
						*Data += Value;
					}
					else
					{
						Tmp.Add(j, Value);
					}
				};

				// Tmp = SparseData - Other.SparseData
				for (const auto& Data : Other.SparseData)
				{
					AddToElement(Data.Key, -Data.Value);
				}

				bool bIsApproxEquals = true;
				for (const auto& Data : Tmp)
				{
					bIsApproxEquals = bIsApproxEquals &&
						(Data.Value < Tolerance) && (-Data.Value < Tolerance);
				}

				return bIsApproxEquals;

			}


		};



		/**
		* Simplifier vertex type that has been extended to include additional sparse data
		*
		* Implements the interface needed by template simplifier code.
		*/

		template< typename BasicAttrContainerType_, typename AttrContainerType_ , typename BoneContainerType_>
		class TSkeletalSimpVert
		{
			typedef TSkeletalSimpVert< BasicAttrContainerType_, AttrContainerType_, BoneContainerType_ >        VertType;

		public:

			typedef BoneContainerType_                                                   BoneContainer;
			typedef AttrContainerType_                                                   AttrContainerType;
			typedef BasicAttrContainerType_                                              BasicAttrContainerType;
			typedef typename BasicAttrContainerType::DenseVecDType                       DenseVecDType;

			// - The "closest" vertex in the source mesh as measured by incremental edge collapse 
			//   i.e. this number should be inherited from the closest vert during edge collapse.
			//   the value -1 will be used as a null-flag
			
			int32 ClosestSrcVertIndex;
			
			// The Vertex Point
			uint32			MaterialIndex;
			FVector3f		Position;

			// Additional weight used to select against collapse.

			float           SpecializedWeight;

			// ---- Vertex Attributes ------------------------------------------------------------------
			//      3 Types:  Dense Attributes, Sparse Attributes,  Bones
			//      Dense & Sparse Attributes are used in quadric calculation
			//      Bones are excluded from the quadric error, but maybe used in imposing penalties for collapse.

			// - Base Attributes: Normal, Tangent, BiTangent, Color, TexCoords: size = 13 + 2 * NumTexCoord
			BasicAttrContainerType  BasicAttributes;

			// - Additional Attributes : size not determined at compile time
			AttrContainerType  AdditionalAttributes;

			// - Sparse Bones : size not determined at compile time
			BoneContainer  SparseBones;

		public:

			typedef typename  BasicAttrContainerType::DenseAttrAccessor    DenseAttrAccessor;


			TSkeletalSimpVert() :
				ClosestSrcVertIndex(-1),
				MaterialIndex(0),
				Position(ForceInitToZero),
				SpecializedWeight(0.f),
				BasicAttributes(),
				AdditionalAttributes(),
				SparseBones()
			{
			}

			// copy constructor
			TSkeletalSimpVert(const TSkeletalSimpVert& Other) :
				ClosestSrcVertIndex(Other.ClosestSrcVertIndex),
				MaterialIndex(Other.MaterialIndex),
				Position(Other.Position),
				SpecializedWeight(Other.SpecializedWeight),
				BasicAttributes(Other.BasicAttributes),
				AdditionalAttributes(Other.AdditionalAttributes),
				SparseBones(Other.SparseBones)
			{}



			uint32 GetMaterialIndex() const { return MaterialIndex; }
			FVector3f& GetPos() { return Position; }
			const FVector3f&	GetPos() const { return Position; }


			// Access to the base attributes.  Note these are really floats. 

			static int32  NumBaseAttributes() { return BasicAttrContainerType::Size(); }
			DenseAttrAccessor GetBasicAttrAccessor() { return BasicAttributes.AsDenseAttrAccessor(); }
			const DenseAttrAccessor GetBasicAttrAccessor() const { return BasicAttributes.AsDenseAttrAccessor(); }

			// Additional attributes maybe dense or sparse.  This should hold bone weights and the like.

			AttrContainerType& GetAdditionalAttrContainer() { return AdditionalAttributes; }
			const AttrContainerType&  GetAdditionalAttrContainer() const { return AdditionalAttributes; }


			// Bones, not used in Quadric calculation.

			BoneContainer& GetSparseBones() { return SparseBones; }
			const BoneContainer&  GetSparseBones() const { return SparseBones; }

			// Insure that the attribute values are valid
			// by correcting any invalid ones.

			void Correct()
			{
				// This fixes the normal, tangent, and bi-tangent.
				BasicAttributes.Correct();

				AdditionalAttributes.Correct();

				SparseBones.Correct();
			}



			TSkeletalSimpVert& operator=(const TSkeletalSimpVert& Other)
			{
				ClosestSrcVertIndex  = Other.ClosestSrcVertIndex;
				MaterialIndex        = Other.MaterialIndex;
				Position             = Other.Position;
				SpecializedWeight    = Other.SpecializedWeight;
				BasicAttributes      = Other.BasicAttributes;
				AdditionalAttributes = Other.AdditionalAttributes;
				SparseBones          = Other.SparseBones;
				
				return *this;
			}

			// Tests approximate equality using specialized 
			// comparison functions.
			// NB: This functionality exists to help in welding verts 
			//     prior to simplification, but is not used in the simplifier itself.
			bool Equals(const VertType& Other) const
			{
				bool bIsApprxEquals = 
					(MaterialIndex == Other.MaterialIndex)
                 && (ClosestSrcVertIndex == Other.ClosestSrcVertIndex)
			     && PointsEqual(Position, Other.Position);

				bIsApprxEquals = bIsApprxEquals
					          && FMath::IsNearlyEqual(SpecializedWeight, Other.SpecializedWeight, 1.e-5);

				bIsApprxEquals = bIsApprxEquals 
					          && BasicAttributes.IsApproxEquals(Other.BasicAttributes);

				bIsApprxEquals = bIsApprxEquals
				 	          && AdditionalAttributes.IsApproxEquals(Other.AdditionalAttributes);

				bIsApprxEquals = bIsApprxEquals
					           && SparseBones.IsApproxEquals(Other.SparseBones);
				return bIsApprxEquals;
			}

			// Exact equality tests

			bool operator==(const VertType& Other) const
			{
				bool bIsEqual = (MaterialIndex == Other.MaterialIndex) && 
					            (ClosestSrcVertIndex == Other.ClosestSrcVertIndex) &&
					            (Position == Other.Position);
				bIsEqual = bIsEqual && (SpecializedWeight == Other.SpecializedWeight);
				bIsEqual = bIsEqual && (GetBasicAttrAccessor() == Other.GetBasicAttrAccessor());
				bIsEqual = bIsEqual && (GetAdditionalAttrContainer() == Other.GetAdditionalAttrContainer());
				bIsEqual = bIsEqual && (SparseBones == Other.SparseBones);

				return bIsEqual;
			}

			// Standard operator overloading.
			// Note: these don't affect the ClosestSrcVertIndex or MaterialIndex

			VertType operator+(const VertType& Other) const
			{
				VertType Result(*this);
				Result.Position += Other.Position;

				Result.SpecializedWeight = FMath::Max(Result.SpecializedWeight, Other.SpecializedWeight);

				auto BaseAttrs = Result.GetBasicAttrAccessor();
				BaseAttrs += Other.GetBasicAttrAccessor();

				AttrContainerType& SparseAttrs = Result.GetAdditionalAttrContainer();
				SparseAttrs += Other.GetAdditionalAttrContainer();

				SparseBones += Other.SparseBones;

				return Result;
			}

			VertType operator-(const VertType& Other) const
			{
				VertType Result(*this);

				Result.Position -= Other.Position;

				Result.SpecializedWeight = FMath::Max(Result.SpecializedWeight, Other.SpecializedWeight);

				auto BaseAttrs = Result.GetBasicAttrAccessor();
				BaseAttrs -= Other.GetBasicAttrAccessor();

				AttrContainerType& SparseAttrs = Result.GetAdditionalAttrContainer();
				SparseAttrs -= Other.GetAdditionalAttrContainer();

				SparseBones -= Other.SparseBones;
				return Result;
			}

			VertType operator*(const float Scalar) const
			{
				VertType Result(*this);
				Result.Position *= Scalar;

				auto BaseAttrs = Result.GetBasicAttrAccessor();
				BaseAttrs *= Scalar;

				AttrContainerType& SparseAttrs = Result.GetAdditionalAttrContainer();
				SparseAttrs *= Scalar;

				BoneContainer& ResultBones = Result.GetSparseBones();

				ResultBones *= Scalar;
				return Result;
			}

			VertType operator/(const float Scalar) const
			{
				float invScalar = 1.0f / Scalar;
				return (*this) * invScalar;
			}
		};
	}
}
