// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
    public class SkeletalMeshReduction : ModuleRules
    {
        public SkeletalMeshReduction(ReadOnlyTargetRules Target) : base(Target)
        {
			StaticAnalyzerDisabledCheckers.Add("core.uninitialized.ArraySubscript");

			// For boost:: and TBB:: code
			//CppCompileWarningSettings.UndefinedIdentifierWarningLevel = WarningLevel.Off;
			//bUseRTTI = true;
			/*
            PublicIncludePaths.AddRange(
                new string[] {
					// ... add public include paths required here ...
				}
                );

            PrivateIncludePaths.AddRange(
                new string[] {
					// ... add other private include paths required here ...
				}
                );
				*/
			PublicDependencyModuleNames.AddRange(
                new string[]
                {
                    "Core",
                    "CoreUObject",
                    "Engine",
                    "InputCore",
                    "UnrealEd",
                    "RawMesh",
                    "MeshUtilities",
                    "MaterialUtilities",
                    "PropertyEditor",
                    "SlateCore",
                    "Slate",                   
                    "RenderCore",
                    "RHI",
                    "QuadricMeshReduction",
					"SkeletalMeshUtilitiesCommon"
					// ... add other public dependencies that you statically link with here ...
				}
                );

            PrivateDependencyModuleNames.AddRange(
                new string[]
                {
					"EditorFramework",
					"AnimationBlueprintLibrary",
                    "MeshBoneReduction",
                    "ClothingSystemRuntimeCommon"
					// ... add private dependencies that you statically link with here ...
                    // QuadricMeshReduction is only for testing
				}
                );
			/*
            DynamicallyLoadedModuleNames.AddRange(
                new string[]
                {
					// ... add any modules that your module loads dynamically here ...
				}
                );
				*/
        }
    }
}
